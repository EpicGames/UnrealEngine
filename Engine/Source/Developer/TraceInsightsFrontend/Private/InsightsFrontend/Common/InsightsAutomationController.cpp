// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsAutomationController.h"

#if INSIGHTS_ENABLE_AUTOMATION

#include "IAutomationControllerModule.h"
#include "IAutomationWorkerModule.h"
#include "ISessionManager.h"
#include "ISessionServicesModule.h"
#include "Misc/App.h"
#include "Misc/AutomationTest.h"
#include "Misc/CoreMisc.h"
#include "Widgets/Docking/SDockTab.h"

#endif

// TraceInsightsCore
#include "InsightsCore/Common/InsightsCoreStyle.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY(InsightsAutomationController);

#define LOCTEXT_NAMESPACE "UE::Insights::FInsightsAutomationController"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Insights
{

const TCHAR* FInsightsAutomationController::AutoQuitMsgOnComplete = TEXT("Application is closing because it was started with the AutoQuit parameter and session analysis is complete and all scheduled tests have completed.");

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsAutomationController::RunTests(const FString& InCmd)
{
#if INSIGHTS_ENABLE_AUTOMATION
	FString ActualCmd = InCmd.Replace(TEXT("\""), TEXT(""));
	if (!ActualCmd.StartsWith(TEXT("Automation RunTests")))
	{
		UE_LOGF(InsightsAutomationController, Warning, "[FInsightsAutomationController] Command %ls does not start with Automation RunTests. Command will be ignored.", *InCmd);
		return;
	}

	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
	IAutomationControllerManagerRef AutomationControllerManager = AutomationControllerModule.GetAutomationController();

	AutomationControllerManager->OnTestsComplete().AddLambda([this]()
		{
			RunningTestsState = ETestsState::Finished;
		});

	RunningTestsState = ETestsState::Running;
	StaticExec(NULL, *ActualCmd);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsAutomationController::Initialize()
{
#if INSIGHTS_ENABLE_AUTOMATION
	FApp::SetSessionName(TEXT("UnrealInsights"));
	ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
	auto SessionService = SessionServicesModule.GetSessionService();
	SessionService->Start();

	// Create Session Manager.
	SessionServicesModule.GetSessionManager();

	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
	AutomationControllerModule.Init();

	// Initialize the target platform manager as it is needed by Automation Window.
	GetTargetPlatformManager();
	FModuleManager::Get().LoadModule("AutomationWindow");
	FModuleManager::Get().LoadModule("AutomationWorker");

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &FInsightsAutomationController::Tick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 1.0f);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsAutomationController::~FInsightsAutomationController()
{
#if INSIGHTS_ENABLE_AUTOMATION
	FTSTicker::RemoveTicker(OnTickHandle);
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FInsightsAutomationController::Tick(float DeltaTime)
{
#if INSIGHTS_ENABLE_AUTOMATION
	if (bAutoQuit && RunningTestsState == ETestsState::Finished)
	{
		RequestEngineExit(AutoQuitMsgOnComplete);
	}

	IAutomationWorkerModule& AutomationWorkerModule = FModuleManager::LoadModuleChecked<IAutomationWorkerModule>("AutomationWorker");
	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));

	AutomationControllerModule.Tick();
	AutomationWorkerModule.Tick();

#endif
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE