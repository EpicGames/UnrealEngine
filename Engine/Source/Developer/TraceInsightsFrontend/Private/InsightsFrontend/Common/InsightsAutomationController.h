// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InsightsFrontend/ITraceInsightsFrontendModule.h"

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Containers/Ticker.h"

#define UE_API TRACEINSIGHTSFRONTEND_API

DECLARE_LOG_CATEGORY_EXTERN(InsightsAutomationController, Log, All);

namespace UE::Insights
{

class FInsightsAutomationController : public TSharedFromThis<FInsightsAutomationController>
{
	enum class ETestsState
	{
		NotStarted,
		Running,
		Finished,
	};

public:
	UE_API virtual ~FInsightsAutomationController();

	UE_API virtual void Initialize();

	UE_API bool Tick(float DeltaTime);

	void SetAutoQuit(bool InAutoQuit) { bAutoQuit = InAutoQuit; }
	bool GetAutoQuit() const { return bAutoQuit; }

	UE_API void RunTests(const FString& InCmd);

private:
	/** The delegate to be invoked when this manager ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	FString CommandToExecute;

	bool bAutoQuit = false;
	ETestsState RunningTestsState = ETestsState::NotStarted;

	static UE_API const TCHAR* AutoQuitMsgOnComplete;
};

} // namespace UE::Insights

#undef UE_API
