// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsFrontend/Common/InsightsFrontendStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "HAL/LowLevelMemTracker.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsFrontendStyle
////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FInsightsFrontendStyle::FStyle> FInsightsFrontendStyle::StyleInstance = nullptr;

////////////////////////////////////////////////////////////////////////////////////////////////////

const ISlateStyle& FInsightsFrontendStyle::Get()
{
	return *StyleInstance;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsFrontendStyle::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("Insights/Frontend/Style"));

	// The UE Core style must be initialized before the InsightsFrontend style
#if WITH_EDITOR
	check(FStarshipCoreStyle::IsInitialized());
#else
	FSlateApplication::InitializeCoreStyle();
#endif

	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<FInsightsFrontendStyle::FStyle> FInsightsFrontendStyle::Create()
{
	TSharedRef<class FInsightsFrontendStyle::FStyle> NewStyle = MakeShareable(new FInsightsFrontendStyle::FStyle(FInsightsFrontendStyle::GetStyleSetName()));
	NewStyle->Initialize();
	return NewStyle;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsFrontendStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FName FInsightsFrontendStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("InsightsFrontendStyle"));
	return StyleSetName;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsFrontendStyle::FStyle
////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsFrontendStyle::FStyle::FStyle(const FName& InStyleSetName)
	: FSlateStyleSet(InStyleSetName)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FInsightsFrontendStyle::FStyle::SyncParentStyles()
{
	const ISlateStyle* ParentStyle = GetParentStyle();

	NormalText = ParentStyle->GetWidgetStyle<FTextBlockStyle>("NormalText");
	Button = ParentStyle->GetWidgetStyle<FButtonStyle>("Button");

	SelectorColor = ParentStyle->GetSlateColor("SelectorColor");
	SelectionColor = ParentStyle->GetSlateColor("SelectionColor");
	SelectionColor_Inactive = ParentStyle->GetSlateColor("SelectionColor_Inactive");
	SelectionColor_Pressed = ParentStyle->GetSlateColor("SelectionColor_Pressed");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#define EDITOR_IMAGE_BRUSH(RelativePath, ...) IMAGE_BRUSH("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_IMAGE_BRUSH_SVG(RelativePath, ...) IMAGE_BRUSH_SVG("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BOX_BRUSH(RelativePath, ...) BOX_BRUSH("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BORDER_BRUSH(RelativePath, ...) BORDER_BRUSH("../../../Editor/Slate/" RelativePath, __VA_ARGS__)
#define TODO_IMAGE_BRUSH(...) EDITOR_IMAGE_BRUSH_SVG("Starship/Common/StaticMesh", __VA_ARGS__)

void FInsightsFrontendStyle::FStyle::Initialize()
{
	SetParentStyleName("InsightsCoreStyle");

	// Sync styles from the parent style that will be used as templates for styles defined here
	SyncParentStyles();

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate/Starship/Insights"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FVector2D Icon12x12(12.0f, 12.0f); // for TreeItem icons
	const FVector2D Icon16x16(16.0f, 16.0f); // for regular icons
	const FVector2D Icon20x20(20.0f, 20.0f); // for ToolBar icons

	Set("AppIcon", new IMAGE_BRUSH_SVG("UnrealInsights", FVector2D(45.0f, 45.0f)));
	Set("AppIconPadding", FMargin(5.0f, 5.0f, 5.0f, 5.0f));

	Set("AppIcon.Small", new IMAGE_BRUSH_SVG("UnrealInsights", FVector2D(24.0f, 24.0f)));
	Set("AppIconPadding.Small", FMargin(4.0f, 4.0f, 0.0f, 0.0f));

	//////////////////////////////////////////////////
	// Trace Store

	Set("Icons.TraceStore", new IMAGE_BRUSH_SVG("TraceStore", Icon16x16));

	Set("Icons.Expand", new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-right", Icon16x16));
	Set("Icons.Expanded", new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-down", Icon16x16));

	Set("Icons.AddWatchDir", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-plus", Icon16x16));
	Set("Icons.RemoveWatchDir", new CORE_IMAGE_BRUSH_SVG("Starship/Common/delete", Icon16x16));

	Set("Icons.Online", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", Icon16x16, FStyleColors::AccentGreen));
	Set("Icons.Offline", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16, FStyleColors::Warning));

	Set("Icons.UTrace", new IMAGE_BRUSH_SVG("UTrace", Icon16x16));
	Set("Icons.UTraceStack", new IMAGE_BRUSH_SVG("UTrace", Icon16x16));

	Set("Icons.TraceServerStart", new CORE_IMAGE_BRUSH_SVG("Starship/Common/play", Icon16x16, FStyleColors::AccentGreen));
	Set("Icons.TraceServerStop", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close", Icon16x16, FStyleColors::AccentRed));

	Set("Icons.Console", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Console", Icon16x16));

	//////////////////////////////////////////////////
	// Connection

	Set("Icons.Connection", new IMAGE_BRUSH_SVG("Connection", Icon16x16));

	//////////////////////////////////////////////////
}

#undef TODO_IMAGE_BRUSH
#undef EDITOR_BOX_BRUSH
#undef EDITOR_IMAGE_BRUSH_SVG
#undef EDITOR_IMAGE_BRUSH

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
