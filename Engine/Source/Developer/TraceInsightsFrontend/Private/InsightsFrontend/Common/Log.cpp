// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsFrontend/Common/Log.h"

DEFINE_LOG_CATEGORY(LogInsightsFrontend);

LLM_DEFINE_TAG(InsightsFrontend);
