// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"

namespace UE::Insights
{

/** Contains all settings for the Unreal Insights, accessible through the main manager. */
class FInsightsFrontendSettings
{
public:
	FInsightsFrontendSettings(bool bInIsDefault = false)
		: bIsEditing(false)
		, bIsDefault(bInIsDefault)
		, bAutoConnect(true)
		, bAutoStartAnalysis(false)
	{
		if (!bIsDefault)
		{
			LoadFromConfig();
		}
	}

	~FInsightsFrontendSettings() = default;

	void LoadFromConfig()
	{
		if (!FConfigContext::ReadIntoGConfig().Load(TEXT("UnrealInsightsFrontendSettings"), SettingsIni))
		{
			return;
		}

		GConfig->GetBool(TEXT("Insights.SessionBrowser"), TEXT("AutoConnect"), bAutoConnect, SettingsIni);
		GConfig->GetBool(TEXT("Insights.SessionBrowser"), TEXT("AutoStartAnalysis"), bAutoStartAnalysis, SettingsIni);
		GConfig->GetString(TEXT("Insights.SessionBrowser"), TEXT("AutoStartAnalysisPlatform"), AutoStartAnalysisPlatform, SettingsIni);
		GConfig->GetString(TEXT("Insights.SessionBrowser"), TEXT("AutoStartAnalysisAppName"), AutoStartAnalysisAppName, SettingsIni);
	}

	void SaveToConfig()
	{
		GConfig->SetBool(TEXT("Insights.SessionBrowser"), TEXT("AutoConnect"), bAutoConnect, SettingsIni);
		GConfig->SetBool(TEXT("Insights.SessionBrowser"), TEXT("AutoStartAnalysis"), bAutoStartAnalysis, SettingsIni);
		GConfig->SetString(TEXT("Insights.SessionBrowser"), TEXT("AutoStartAnalysisPlatform"), *AutoStartAnalysisPlatform, SettingsIni);
		GConfig->SetString(TEXT("Insights.SessionBrowser"), TEXT("AutoStartAnalysisAppName"), *AutoStartAnalysisAppName, SettingsIni);

		GConfig->Flush(false, SettingsIni);
	}

	void EnterEditMode()
	{
		bIsEditing = true;
	}

	void ExitEditMode()
	{
		bIsEditing = false;
	}

	const bool IsEditing() const
	{
		return bIsEditing;
	}

	const FInsightsFrontendSettings& GetDefaults() const
	{
		return Defaults;
	}

	void ResetToDefaults()
	{
		bAutoConnect = Defaults.bAutoConnect;
	}

	#define SET_AND_SAVE(Option, Value) { if (Option != Value) { Option = Value; SaveToConfig(); } }

	bool IsAutoConnectEnabled() const { return bAutoConnect; }
	void SetAutoConnect(bool bOnOff) { bAutoConnect = bOnOff; }
	void SetAndSaveAutoConnect(bool bOnOff) { SET_AND_SAVE(bAutoConnect, bOnOff); }

	bool IsAutoStartAnalysisEnabled() const { return bAutoStartAnalysis; }
	void SetAutoStartAnalysis(bool bOnOff) { bAutoStartAnalysis = bOnOff; }
	void SetAndSaveAutoStartAnalysis(bool bOnOff) { SET_AND_SAVE(bAutoStartAnalysis, bOnOff); }

	FString GetAutoStartAnalysisPlatform() const { return AutoStartAnalysisPlatform; }
	void SetAutoStartAnalysisPlatform(const FString& Value) { AutoStartAnalysisPlatform = Value; }
	void SetAndSaveAutoStartAnalysisPlatform(const FString& Value) { SET_AND_SAVE(AutoStartAnalysisPlatform, Value); }

	FString GetAutoStartAnalysisAppName() const { return AutoStartAnalysisAppName; }
	void SetAutoStartAnalysisAppName(const FString& Value) { AutoStartAnalysisAppName = Value; }
	void SetAndSaveAutoStartAnalysisAppName(const FString& Value) { SET_AND_SAVE(AutoStartAnalysisAppName, Value); }

	#undef SET_AND_SAVE

private:
	/** Contains default settings. */
	static FInsightsFrontendSettings Defaults;

	/** Setting filename ini. */
	FString SettingsIni;

	/** Whether profiler settings is in edit mode. */
	bool bIsEditing;

	/** Whether this instance contains defaults. */
	bool bIsDefault;

	//////////////////////////////////////////////////
	// Actual settings.

	/** Whether Insights should signal to the Editor to auto connect and start tracing when Insights is running */
	bool bAutoConnect;

	/** Whether Insights should auto start analysis when a new live trace in received.*/
	bool bAutoStartAnalysis;

	/** If specified, auto-start analysis will be enabled only for live trace sessions with this specified Platform.*/
	FString AutoStartAnalysisPlatform;

	/** If specified, auto-start analysis will be enabled only for live trace sessions with this specified App Name.*/
	FString AutoStartAnalysisAppName;
};

} // namespace UE::Insights
