// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiagnosticsSessionAnalyzer.h"

#include "HAL/LowLevelMemTracker.h"

// TraceServices
#include "TraceServices/Model/Diagnostics.h"

namespace UE::Insights
{

void FDiagnosticsSessionAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Session, "Diagnostics", "Session");
	Builder.RouteEvent(RouteId_Session2, "Diagnostics", "Session2");
}

bool FDiagnosticsSessionAnalyzer::OnEvent(uint16 RouteId, EStyle, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FDiagnosticsSessionAnalyzer"));

	const FEventData& EventData = Context.EventData;

	switch (RouteId)
	{
	case RouteId_Session:
	{
		const uint8* Attachment = EventData.GetAttachment();
		if (Attachment == nullptr)
		{
			return false;
		}

		uint8 AppNameOffset = EventData.GetValue<uint8>("AppNameOffset");
		uint8 CommandLineOffset = EventData.GetValue<uint8>("CommandLineOffset");

		Platform = FString::ConstructFromPtrSize((const ANSICHAR*)Attachment, AppNameOffset);

		Attachment += AppNameOffset;
		int32 AppNameLength = CommandLineOffset - AppNameOffset;
		AppName = FString::ConstructFromPtrSize((const ANSICHAR*)Attachment, AppNameLength);

		Attachment += AppNameLength;
		int32 CommandLineLength = EventData.GetAttachmentSize() - CommandLineOffset;
		CommandLine = FString::ConstructFromPtrSize((const ANSICHAR*)Attachment, CommandLineLength);

		ConfigurationType = (EBuildConfiguration) EventData.GetValue<uint8>("ConfigurationType");
		TargetType = (EBuildTargetType) EventData.GetValue<uint8>("TargetType");

		return false;
	}
	case RouteId_Session2:
	{
		EventData.GetString("Platform", Platform);
		EventData.GetString("AppName", AppName);
		EventData.GetString("ProjectName", ProjectName);
		EventData.GetString("BuildVersion", BuildVersion);
		EventData.GetString("Branch", Branch);
		EventData.GetString("EngineVersion", EngineVersion);
		EventData.GetString("CommandLine", CommandLine);
		EventData.GetString("VFSPaths", VFSPaths);
		Changelist = EventData.GetValue<uint32>("Changelist", 0);
		ConfigurationType = (EBuildConfiguration)EventData.GetValue<uint8>("ConfigurationType");
		TargetType = (EBuildTargetType)EventData.GetValue<uint8>("TargetType");
		return false;
	}
	} // switch

	return true;
}

} // namespace UE::Insights
