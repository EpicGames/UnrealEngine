// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// TraceAnalysis
#include "Trace/Analyzer.h"

namespace UE::Insights
{

struct FDiagnosticsSessionAnalyzer : public UE::Trace::IAnalyzer
{
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle, const FOnEventContext& Context) override;

	enum : uint16
	{
		RouteId_Session,
		RouteId_Session2,
	};

	FString Platform;
	FString AppName;
	FString ProjectName;
	FString BuildVersion;
	FString Branch;
	FString EngineVersion;
	FString CommandLine;
	FString VFSPaths;
	uint32 Changelist = 0;
	EBuildConfiguration ConfigurationType = EBuildConfiguration::Unknown;
	EBuildTargetType TargetType = EBuildTargetType::Unknown;
};

} // namespace UE::Insights
