// Copyright Epic Games, Inc. All Rights Reserved.

#include "StoreBrowser.h"

#include "Async/ParallelFor.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Misc/Paths.h"

// TraceAnalysis
#include "Trace/Analysis.h"
#include "Trace/StoreClient.h"
#include "Trace/StoreConnection.h"

// TraceInsightsCore
#include "InsightsCore/Common/Stopwatch.h"

// TraceInsights
//#include "Insights/InsightsManager.h"
#include "InsightsFrontend/Common/Log.h"
#include "InsightsFrontend/StoreService/DiagnosticsSessionAnalyzer.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FStoreBrowser::FStoreBrowser(TSharedRef<UE::Trace::FStoreConnection> InTraceStoreConnection)
: TraceStoreConnection(InTraceStoreConnection)
{
	bRunning = true;
	Thread = FRunnableThread::Create(this, TEXT("StoreBrowser"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FStoreBrowser::~FStoreBrowser()
{
	Stop();

	if (Thread)
	{
		Thread->Kill(true);
		Thread = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FStoreBrowser::Init()
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FStoreBrowser::Run()
{
	while (bRunning)
	{
		FPlatformProcess::SleepNoStats(0.5f);
		UpdateTraces();
	}
	return 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStoreBrowser::Stop()
{
	bRunning = false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStoreBrowser::Exit()
{
	ResetTraces();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStoreBrowser::Refresh()
{
	if (AreSettingsLocked() || AreTracesLocked())
	{
		return;
	}

	StoreChangeSerial = 0;
	StoreSettingsChangeSerial = 0;

	{
		FScopeLock Lock(&SettingsCriticalSection);
		SettingsChangeSerial = 0;
		Host.Reset();
		Version.Reset();
		StoreDirectory.Reset();
		WatchDirectories.Reset();
	}

	{
		FScopeLock Lock(&TracesCriticalSection);
		TracesChangeSerial = 0;
		for (const TSharedPtr<FStoreBrowserTraceInfo>& Trace : Traces)
		{
			Trace->MetadataUpdateCount = 0;
		}
		Traces.Reset();
		TraceMap.Reset();
		LiveTraceMap.Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStoreBrowser::UpdateTraces()
{
	UE::Trace::FStoreClient* StoreClient = TraceStoreConnection->GetStoreClient();
	if (!StoreClient)
	{
		ResetTraces();
		return;
	}

	UE::Insights::FStopwatch StopwatchTotal;
	StopwatchTotal.Start();

	// Update store host.
	if (Host.IsEmpty())
	{
		FScopeLock Lock(&SettingsCriticalSection);
		SettingsChangeSerial++;
		Host = TraceStoreConnection->GetLastStoreHost();
	}

	// Check if connection to store is still active We want to do this without locking the critical
	// section, in case the UI needs to read values. The output is an atomic anyway.
	{
		EConnectionStatus PreviousConnectionStatus = ConnectionStatus;

		bool bIsValid = StoreClient->IsValid();
		while (!bIsValid && bRunning) // TODO: Perhaps a max reconnection attempts is needed?
		{
			constexpr EConnectionStatus ReconnectionFrequency = static_cast<EConnectionStatus>(5);
			uint32 SecondsToSleep = static_cast<uint32>(ReconnectionFrequency);
			do
			{
				FPlatformProcess::Sleep(1.0f);
				ConnectionStatus.store(static_cast<EConnectionStatus>(--SecondsToSleep));
			} while (SecondsToSleep);

			// At this point ConnectionStatus is zero (EConnectionStatus::Connecting)
			bIsValid = TraceStoreConnection->ReconnectToStore();

			if (bIsValid)
			{
				// If we just reconnected we need to reset known traces and change serials
				Refresh();
			}
		}
		ConnectionStatus.store(bIsValid ? EConnectionStatus::Connected : EConnectionStatus::Connecting);

		if (!bIsValid)
		{
			// If there is no connection, no point in checking for sessions
			return;
		}

		if (ConnectionStatus == EConnectionStatus::Connected &&
			(PreviousConnectionStatus != EConnectionStatus::Connected || StoreChangeSerial == 0))
		{
			// Get the server version.
			FString StoreVersion;
			{
				FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
				const auto VersionResult = StoreClient->GetVersion();

				if (VersionResult)
				{
					TStringBuilder<128> VersionString;
					VersionString << VersionResult->GetMajorVersion() << TEXT(".") << VersionResult->GetMinorVersion();
					if (!VersionResult->GetConfiguration().Equals("Release"))
					{
						VersionString << TEXT(" (") << VersionResult->GetConfiguration() << TEXT(")");
					}
					StoreVersion = VersionString.ToString();
				}
				else
				{
					StoreVersion = TEXT("Unknown");
				}
			}

			// Update settings.
			{
				FScopeLock Lock(&SettingsCriticalSection);
				SettingsChangeSerial++;
				Host = TraceStoreConnection->GetLastStoreHost();
				Version = StoreVersion;
			}
		}
	}

	// Check if the list of trace sessions or store settings have changed.
	{
		uint32 NewStoreChangeSerial = 0;
		uint32 NewStoreSettingsChangeSerial = 0;
		FString NewStoreDirectory;
		TArray<FString> NewWatchDirectories;
		TOptional<uint32> NewStorePort, NewRecorderPort;

		{
			// Get Trace Store status.
			FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
			const UE::Trace::FStoreClient::FStatus* Status = StoreClient->GetStatus();
			if (Status)
			{
				NewStoreChangeSerial = Status->GetChangeSerial();
				NewStoreSettingsChangeSerial = Status->GetSettingsSerial();
				if (StoreSettingsChangeSerial != NewStoreSettingsChangeSerial)
				{
					NewStoreDirectory = FString(Status->GetStoreDir());
					Status->GetWatchDirectories(NewWatchDirectories);
					NewStorePort = Status->GetStorePort();
					NewRecorderPort = Status->GetRecorderPort();
				}
			}
		}

		if (StoreSettingsChangeSerial != NewStoreSettingsChangeSerial)
		{
			StoreSettingsChangeSerial = NewStoreSettingsChangeSerial;

			// Update settings.
			FScopeLock Lock(&SettingsCriticalSection);
			SettingsChangeSerial++;
			StoreDirectory = NewStoreDirectory;
			WatchDirectories.Empty();
			WatchDirectories = NewWatchDirectories;
			StorePort = NewStorePort.Get(StorePort);
			RecorderPort = NewRecorderPort.Get(RecorderPort);
		}

		if (StoreChangeSerial != NewStoreChangeSerial)
		{
			StoreChangeSerial = NewStoreChangeSerial;

			// Check for removed traces.
			{
				TSet<uint32, DefaultKeyFuncs<uint32>, TInlineSetAllocator<4>> RemovedTraces;
				for (const TSharedPtr<FStoreBrowserTraceInfo>& Trace : Traces)
				{
					FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
					const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(Trace->TraceId);
					if (TraceInfo == nullptr)
					{
						RemovedTraces.Add(Trace->TraceId);
					}
				}
				if (RemovedTraces.Num() > 0)
				{
					FScopeLock Lock(&TracesCriticalSection);
					TracesChangeSerial++;
					Traces.RemoveAllSwap([&RemovedTraces](const TSharedPtr<FStoreBrowserTraceInfo>& Trace) { return RemovedTraces.Contains(Trace->TraceId); });
					for (const uint32& TraceId : RemovedTraces)
					{
						TraceMap.Remove(TraceId);
					}
				}
			}

			// Check for added traces.
			{
				TArray<TSharedPtr<FStoreBrowserTraceInfo>, TInlineAllocator<8>> AddedTraces;
				int32 TraceCount = 0;
				{
					FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
					TraceCount = StoreClient->GetTraceCount();
				}
				for (int32 TraceIndex = 0; TraceIndex < TraceCount; ++TraceIndex)
				{
					FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
					const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfo(TraceIndex);
					if (TraceInfo != nullptr)
					{
						const uint32 TraceId = TraceInfo->GetId();
						if (!TraceMap.Find(TraceId))
						{
							TSharedPtr<FStoreBrowserTraceInfo> TracePtr = MakeShared<FStoreBrowserTraceInfo>();
							FStoreBrowserTraceInfo& Trace = *TracePtr;

							Trace.TraceId = TraceId;

							const FUtf8StringView Utf8NameView = TraceInfo->GetName();
							Trace.Name = FString(Utf8NameView);
							FUtf8StringView Uri = TraceInfo->GetUri();
							if (Uri.Len() > 0)
							{
								Trace.Uri = FString(Uri);
							}
							else
							{
								// Fallback for older versions of UTS which didn't write URI.
								Trace.Uri = FPaths::SetExtension(FPaths::Combine(StoreDirectory, Trace.Name), TEXT(".utrace"));
								FPaths::MakePlatformFilename(Trace.Uri);
							}

							Trace.Timestamp = FStoreBrowserTraceInfo::ConvertTimestamp(TraceInfo->GetTimestamp());
							Trace.Size = TraceInfo->GetSize();

							AddedTraces.Add(TracePtr);

							// Flush list from time to time.
							if (AddedTraces.Num() > 5)
							{
								StoreClientLock.Unlock();
								FScopeLock Lock(&TracesCriticalSection);
								TracesChangeSerial++;
								for (const TSharedPtr<FStoreBrowserTraceInfo>& AddedTracePtr : AddedTraces)
								{
									Traces.Add(AddedTracePtr);
									TraceMap.Add(AddedTracePtr->TraceId, AddedTracePtr);
								}
								AddedTraces.Reset();
							}
						}
					}
				}
				if (AddedTraces.Num() > 0)
				{
					FScopeLock Lock(&TracesCriticalSection);
					TracesChangeSerial++;
					for (TSharedPtr<FStoreBrowserTraceInfo> AddedTracePtr : AddedTraces)
					{
						Traces.Add(AddedTracePtr);
						TraceMap.Add(AddedTracePtr->TraceId, AddedTracePtr);
					}
				}
			}
		}
	}

	StopwatchTotal.Update();
	const uint64 Step1 = StopwatchTotal.GetAccumulatedTimeMs();

	// Update the live trace sessions.
	{
		TArray<uint32> NotLiveTraces;

		for (const auto& KV : LiveTraceMap)
		{
			const uint32 TraceId = KV.Key;
			FStoreBrowserTraceInfo& Trace = *KV.Value;

			FDateTime Timestamp(0);
			uint64 Size = 0;
			ensure(Trace.bIsLive);
			bool bIsLive = true;
			uint32 IpAddress = 0;

			FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());

			const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(TraceId);
			if (TraceInfo != nullptr)
			{
				Timestamp = FStoreBrowserTraceInfo::ConvertTimestamp(TraceInfo->GetTimestamp());
				Size = TraceInfo->GetSize();
			}

			const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfoByTraceId(TraceId);
			if (SessionInfo != nullptr)
			{
				bIsLive = true;
				IpAddress = SessionInfo->GetIpAddress();
			}
			else
			{
				bIsLive = false;
			}

			StoreClientLock.Unlock();

			if (!bIsLive || IpAddress != Trace.IpAddress || Timestamp != Trace.Timestamp || Size != Trace.Size)
			{
				StoreClientLock.Unlock();

				FScopeLock Lock(&TracesCriticalSection);
				TracesChangeSerial++;
				Trace.ChangeSerial++;
				Trace.Timestamp = Timestamp;
				Trace.Size = Size;
				Trace.bIsLive = bIsLive;
				Trace.IpAddress = IpAddress;
			}

			if (!bIsLive)
			{
				NotLiveTraces.Add(TraceId);
			}
		}

		for (const uint32 TraceId : NotLiveTraces)
		{
			LiveTraceMap.Remove(TraceId);
		}
	}

	StopwatchTotal.Update();
	const uint64 Step2 = StopwatchTotal.GetAccumulatedTimeMs();

	// Check if we have new live sessions.
	{
		uint32 SessionCount = 0;
		{
			FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
			SessionCount = StoreClient->GetSessionCount();
		}
		for (uint32 SessionIndex = 0; SessionIndex < SessionCount; ++SessionIndex)
		{
			FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
			const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(SessionIndex);
			if (SessionInfo != nullptr)
			{
				const uint32 TraceId = SessionInfo->GetTraceId();
				if (!LiveTraceMap.Find(TraceId))
				{
					TSharedPtr<FStoreBrowserTraceInfo>* TracePtrPtr = TraceMap.Find(TraceId);
					if (TracePtrPtr)
					{
						// This trace is a new live session.
						LiveTraceMap.Add(TraceId, *TracePtrPtr);

						const uint32 IpAddress = SessionInfo->GetIpAddress();

						FDateTime Timestamp(0);
						uint64 Size = 0;
						const UE::Trace::FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(TraceId);
						if (TraceInfo != nullptr)
						{
							Timestamp = FStoreBrowserTraceInfo::ConvertTimestamp(TraceInfo->GetTimestamp());
							Size = TraceInfo->GetSize();
						}

						FStoreBrowserTraceInfo& Trace = **TracePtrPtr;

						StoreClientLock.Unlock();
						FScopeLock Lock(&TracesCriticalSection);
						TracesChangeSerial++;
						Trace.ChangeSerial++;
						Trace.Timestamp = Timestamp;
						Trace.Size = Size;
						Trace.bIsLive = true;
						Trace.IpAddress = IpAddress;
					}
				}
			}
		}
	}

	StopwatchTotal.Update();
	const uint64 Step3 = StopwatchTotal.GetAccumulatedTimeMs();

	// Check to see if we need to update metadata.
	{
		UE::Insights::FStopwatch Stopwatch;
		Stopwatch.Start();

		TArray<TSharedPtr<FStoreBrowserTraceInfo>> TracesToUpdate;

		for (const TSharedPtr<FStoreBrowserTraceInfo>& TraceInfo : Traces)
		{
			if (TraceInfo->MetadataUpdateCount > 0)
			{
				TracesToUpdate.Add(TraceInfo);
			}
		}

		// Sort descending by timestamp (i.e. to update the newer traces first).
		TracesToUpdate.Sort([](const TSharedPtr<FStoreBrowserTraceInfo>& A, const TSharedPtr<FStoreBrowserTraceInfo>& B)
			{
				return A->Timestamp.GetTicks() > B->Timestamp.GetTicks();
			});

#if 0
		for (TSharedPtr<FStoreBrowserTraceInfo>& Trace : TracesToUpdate)
		{
			UpdateMetadata(Trace);
		}
#else
		if (TracesToUpdate.Num() > 0)
		{
			UE_LOGF(LogInsightsFrontend, Log, "[StoreBrowser] Starting metadata update for %d trace(s).", TracesToUpdate.Num());
			ParallelFor(TracesToUpdate.Num(), [this, &TracesToUpdate](uint32 Index)
				{
					UpdateMetadata(TracesToUpdate[Index]);
				},
				EParallelForFlags::BackgroundPriority);
		}
#endif

		Stopwatch.Stop();
		if (TracesToUpdate.Num() > 0)
		{
			UE_LOGF(LogInsightsFrontend, Log, "[StoreBrowser] Metadata updated in %llu ms for %d trace(s) (~%llu ms/trace).",
				Stopwatch.GetAccumulatedTimeMs(),
				TracesToUpdate.Num(),
				Stopwatch.GetAccumulatedTimeMs() / TracesToUpdate.Num());
		}
	}

	StopwatchTotal.Stop();
	const uint64 TotalTime = StopwatchTotal.GetAccumulatedTimeMs();
	if (TotalTime > 100)
	{
		UE_LOGF(LogInsightsFrontend, Log, "[StoreBrowser] Updated in %llu ms (%llu + %llu + %llu + %llu).",
			TotalTime, Step1, Step2 - Step1, Step3 - Step2, TotalTime - Step3);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStoreBrowser::ResetTraces()
{
	if (Traces.Num() > 0)
	{
		FScopeLock Lock(&TracesCriticalSection);
		TracesChangeSerial++;
		for (const TSharedPtr<FStoreBrowserTraceInfo>& Trace : Traces)
		{
			Trace->MetadataUpdateCount = 0;
		}
		Traces.Reset();
		TraceMap.Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FStoreBrowser::UpdateMetadata(TSharedPtr<FStoreBrowserTraceInfo> TraceInfoPtr)
{
	FStoreBrowserTraceInfo& Trace = *TraceInfoPtr;
	if (Trace.MetadataUpdateCount == 0)
	{
		return;
	}

	UE::Trace::FStoreClient* StoreClient = TraceStoreConnection->GetStoreClient();
	if (!StoreClient)
	{
		return;
	}

	FPlatformProcess::SleepNoStats(0.0f);

	FScopeLock StoreClientLock(&TraceStoreConnection->GetStoreClientCriticalSection());
	UE::Trace::FStoreClient::FTraceData TraceData = StoreClient->ReadTrace(Trace.TraceId);
	StoreClientLock.Unlock();
	if (!TraceData)
	{
		return;
	}

	struct FDataStream : public UE::Trace::IInDataStream
	{
		enum class EReadStatus
		{
			Ready = 0,
			StoppedByReadSizeLimit,
			StoppedByTimeLimit,
			Cancelled
		};

		virtual int32 Read(void* Data, uint32 Size) override
		{
			if (BytesRead >= 1024 * 1024)
			{
				Status = EReadStatus::StoppedByReadSizeLimit;
				return 0;
			}

			if (Trace->MetadataUpdateCount == 0)
			{
				Status = EReadStatus::Cancelled;
				return 0;
			}

			Stopwatch.Update();
			if (Stopwatch.GetAccumulatedTime() > TimeLimit)
			{
				Status = EReadStatus::StoppedByTimeLimit;
				return 0;
			}

			const int32 InnerBytesRead = Inner->Read(Data, Size);
			BytesRead += InnerBytesRead;

			return InnerBytesRead;
		}

		virtual void Close() override
		{
			Inner->Close();
		}

		UE::Trace::IInDataStream* Inner;
		FStoreBrowserTraceInfo* Trace;
		double TimeLimit = 1.0; // [seconds]
		UE::Insights::FStopwatch Stopwatch;
		int32 BytesRead = 0;
		EReadStatus Status = EReadStatus::Ready;
	};

	FDataStream DataStream;
	DataStream.Inner = TraceData.Get();
	DataStream.Trace = &Trace;
	DataStream.TimeLimit = 1.0 + (double)(Trace.MetadataUpdateCount - 1) * 2.0; // 1s, 3s, 5s, ...
	DataStream.Stopwatch.Start();

	FDiagnosticsSessionAnalyzer Analyzer;
	UE::Trace::FAnalysisContext Context;
	Context.AddAnalyzer(Analyzer);
	Context.Process(DataStream).Wait();

	TraceData.Reset();
	DataStream.Inner = nullptr;

	// Update the FStoreBrowserTraceInfo object.
	if (Trace.MetadataUpdateCount != 0)
	{
		FScopeLock Lock(&TracesCriticalSection);

		if (Analyzer.Platform.Len() != 0)
		{
			TracesChangeSerial++;
			Trace.ChangeSerial++;
			Trace.Platform = Analyzer.Platform;
			Trace.AppName = Analyzer.AppName;
			Trace.ProjectName = Analyzer.ProjectName;
			Trace.ConfigurationType = static_cast<EBuildConfiguration>(Analyzer.ConfigurationType);
			Trace.TargetType = static_cast<EBuildTargetType>(Analyzer.TargetType);
			Trace.BuildVersion = Analyzer.BuildVersion;
			Trace.Branch = Analyzer.Branch;
			Trace.Changelist = Analyzer.Changelist;
			Trace.EngineVersion = Analyzer.EngineVersion;
			Trace.CommandLine = Analyzer.CommandLine;
			Trace.VFSPaths = Analyzer.VFSPaths;
		}

		if (DataStream.Status == FDataStream::EReadStatus::StoppedByTimeLimit)
		{
			// Try again later.
			++Trace.MetadataUpdateCount;
			if (Trace.MetadataUpdateCount > 5)
			{
				Trace.MetadataUpdateCount = 0; // no more updates
			}
		}
		else
		{
			Trace.MetadataUpdateCount = 0; // no more updates
		}
	}

	DataStream.Stopwatch.Stop();
	UE_LOGF(LogInsightsFrontend, Log, "[StoreBrowser] Metadata updated in %llu ms for trace \"%ls\"",
		DataStream.Stopwatch.GetAccumulatedTimeMs(), *Trace.Name);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
