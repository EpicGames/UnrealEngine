// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

namespace UE::Trace
{
	class FStoreClient;
	class FStoreConnection;
}

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FStoreBrowserTraceInfo
{
	uint32 TraceId = 0;

	uint64 ChangeSerial = 0;

	FString Name;
	FString Uri;

	FDateTime Timestamp = 0;
	uint64 Size = 0;

	FString Platform;
	FString AppName;
	FString ProjectName;
	FString BuildVersion;
	FString Branch;
	FString EngineVersion;
	FString CommandLine;
	FString VFSPaths;
	uint32 Changelist;
	EBuildConfiguration ConfigurationType = EBuildConfiguration::Unknown;
	EBuildTargetType TargetType = EBuildTargetType::Unknown;

	bool bIsLive = false;
	std::atomic<uint8> MetadataUpdateCount = 1;

	uint32 IpAddress = 0;

	FStoreBrowserTraceInfo() = default;

	static FDateTime ConvertTimestamp(uint64 InTimestamp)
	{
		return FDateTime(static_cast<int64>(InTimestamp));
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FStoreBrowser : public FRunnable
{
	friend class FStoreBrowserUpdaterAsyncTask;

public:
	explicit FStoreBrowser(TSharedRef<UE::Trace::FStoreConnection> InTraceStoreConnection);
	virtual ~FStoreBrowser();

	//////////////////////////////////////////////////
	// FRunnable interface

	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

	//////////////////////////////////////////////////

	enum class EConnectionStatus : uint8
	{
		// Attempting connection
		Connecting = 0,
		// Values between (start,end) is interpreted as
		// number of seconds until next reconnection attempt
		SecondsToReconnectStart,
		SecondsToReconnectEnd = 0xfd,
		// No connection could be made, no more reconnection
		// attempts are made.
		NoConnection = 0xfe,
		// Connection is active
		Connected = 0xff
	};

	bool IsRunning() const { return bRunning; }
	EConnectionStatus GetConnectionStatus() const { return ConnectionStatus; }

	bool AreSettingsLocked() const { return bSettingsLocked; }
	void LockSettings() { check(!bSettingsLocked); bSettingsLocked = true; SettingsCriticalSection.Lock(); }
	void UnlockSettings() { check(bSettingsLocked); bSettingsLocked = false; SettingsCriticalSection.Unlock(); }
	uint32 GetSettingsChangeSerial() const { check(bSettingsLocked); return SettingsChangeSerial; };
	const FString& GetHost() const { check(bSettingsLocked); return Host; }
	const FString& GetVersion() const { check(bSettingsLocked); return Version; }
	uint32 GetRecorderPort() const {check(bSettingsLocked); return RecorderPort; }
	uint32 GetStorePort() const { check(bSettingsLocked); return StorePort; }
	const FString& GetStoreDirectory() const { check(bSettingsLocked); return StoreDirectory; }
	const TArray<FString>& GetWatchDirectories() const { check(bSettingsLocked); return WatchDirectories; }

	bool AreTracesLocked() const { return bTracesLocked; }
	void LockTraces() { check(!bTracesLocked); bTracesLocked = true; TracesCriticalSection.Lock(); }
	void UnlockTraces() { check(bTracesLocked); bTracesLocked = false; TracesCriticalSection.Unlock(); }
	uint32 GetTracesChangeSerial() const { check(bTracesLocked); return TracesChangeSerial; }
	const TArray<TSharedPtr<FStoreBrowserTraceInfo>>& GetTraces() const { check(bTracesLocked); return Traces; }
	const TMap<uint32, TSharedPtr<FStoreBrowserTraceInfo>>& GetTraceMap() const { check(bTracesLocked); return TraceMap; }

	void Refresh();

private:
	void UpdateTraces();
	void ResetTraces();

	void UpdateMetadata(TSharedPtr<FStoreBrowserTraceInfo> TraceInfoPtr);

private:
	// Thread safe bool for stopping the thread
	std::atomic<bool> bRunning = false;

	// Thread for continuously updating and caching info about trace store.
	FRunnableThread* Thread = nullptr;

	TSharedRef<UE::Trace::FStoreConnection> TraceStoreConnection;
	std::atomic<EConnectionStatus> ConnectionStatus = EConnectionStatus::Connecting;

	uint32 StoreChangeSerial = 0;
	uint32 StoreSettingsChangeSerial = 0;

	mutable FCriticalSection SettingsCriticalSection;
	std::atomic<bool> bSettingsLocked = false; // for debugging, to ensure Get*() methods for settings are only called between Lock() - Unlock() calls.
	uint32 SettingsChangeSerial = 0;
	FString Host;
	FString Version;
	uint32 StorePort;
	uint32 RecorderPort;
	FString StoreDirectory;
	TArray<FString> WatchDirectories;

	mutable FCriticalSection TracesCriticalSection;
	std::atomic<bool> bTracesLocked = false; // for debugging, to ensure Get*() methods for traces are only called between Lock() - Unlock() calls.
	uint32 TracesChangeSerial = 0;
	TArray<TSharedPtr<FStoreBrowserTraceInfo>> Traces;
	TMap<uint32, TSharedPtr<FStoreBrowserTraceInfo>> TraceMap;
	TMap<uint32, TSharedPtr<FStoreBrowserTraceInfo>> LiveTraceMap;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
