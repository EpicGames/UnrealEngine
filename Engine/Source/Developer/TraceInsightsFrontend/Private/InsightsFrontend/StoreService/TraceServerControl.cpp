// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsFrontend/StoreService/TraceServerControl.h"

#include "CoreGlobals.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "SlateOptMacros.h"

// TraceAnalysis
#include "Trace/StoreClient.h"

// TraceInsightsCore
#include "InsightsCore/Common/Log.h"

/////////////////////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "UE::Insights::FTraceServerControl"

namespace UE::Insights
{

// Number of times to attempt state change before failing
constexpr uint32 GStateChangeRetries = 6;
// Number of times to attempt reconnection while starting server
constexpr uint32 GStartConnectAttempts = 5;
// Number of seconds between each reconnection attempt
constexpr float GStartConnectFrequencySeconds = 0.5;

/////////////////////////////////////////////////////////////////////////////////////////

FTraceServerControl::FTraceServerControl(const TCHAR* InHost, uint32 InPort, FName InStyleSet)
	: Host(InHost)
	, Port(InPort)
	, StyleSet(InStyleSet)
{
	bIsLocalHost = Host.Equals(TEXT("127.0.0.1")) || Host.Equals(TEXT("localhost"));
}

/////////////////////////////////////////////////////////////////////////////////////////

FTraceServerControl::~FTraceServerControl()
{
	bIsCancelRequested = true;
	FScopeLock _(&AsyncTaskLock); // wait for async tasks to complete
}

/////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FTraceServerControl::MakeMenu(FMenuBuilder& Builder)
{
	// Create client
	if (!Client)
	{
		Client = (UE::Trace::FStoreClient::Create(*Host, Port));
		if (Client)
		{
			ChangeState(EState::NotConnected, EState::Connected);
		}
	}

	// If connected kick off status and version check
	if (State.load(std::memory_order_relaxed) == EState::Connected)
	{
		TriggerStatusUpdate();
	}

	if (bIsLocalHost)
	{
		Builder.BeginSection("LocalTraceServer", LOCTEXT("Section_LocalServer", "Local Trace Server"));
	}
	else
	{
		Builder.BeginSection("TraceServer", FText::Format(LOCTEXT("Section_RemoteServer", "Remote Trace Server '{0}'"), FText::FromString(Host)));
	}
	Builder.AddMenuEntry(
		TAttribute<FText>::CreateLambda([this] { FScopeLock _(&StringsLock); return FText::FromString(StatusString.IsEmpty() ? TEXT("Not running") : StatusString); }),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(FExecuteAction(), FCanExecuteAction::CreateLambda([]() { return false; })),
		NAME_None,
		EUserInterfaceActionType::Button
	);
	if (bIsLocalHost)
	{
		Builder.AddMenuEntry(
			LOCTEXT("ServerControlSponsoredLabel", "Sponsored Mode"),
			LOCTEXT("ServerControlSponsoredTooltip", "In sponsored mode the server only runs as long as local processes that uses it are alive."),
			FSlateIcon(), //?
			FUIAction(
				FExecuteAction::CreateRaw(this, &FTraceServerControl::OnSponsored_Changed),
				FCanExecuteAction::CreateRaw(this, &FTraceServerControl::AreControlsEnabled),
				FGetActionCheckState::CreateLambda([this](){ return IsSponsored() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		Builder.AddMenuEntry(
			LOCTEXT("ServerControlStartLabel", "Start"),
			LOCTEXT("ServerControlStartTooltip", "Starts the Trace Server"),
			FSlateIcon(StyleSet, "Icons.TraceServerStart"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FTraceServerControl::OnStart_Clicked),
				FCanExecuteAction::CreateRaw(this, &FTraceServerControl::CanServerBeStarted)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		Builder.AddMenuEntry(
			LOCTEXT("ServerControlStopLabel", "Stop"),
			LOCTEXT("ServerControlStopTooltip", "Stops the Trace Server. Any running traces will be canceled."),
			FSlateIcon(StyleSet, "Icons.TraceServerStop"),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FTraceServerControl::OnStop_Clicked),
				FCanExecuteAction::CreateRaw(this, &FTraceServerControl::CanServerBeStopped)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	Builder.EndSection();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* LexState(FTraceServerControl::EState state)
{
	switch (state)
	{
		case FTraceServerControl::EState::NotConnected: return TEXT("NotConnected");
		case FTraceServerControl::EState::Connecting: return TEXT("Connecting");
		case FTraceServerControl::EState::Connected: return TEXT("Connected");
		case FTraceServerControl::EState::CheckStatus: return TEXT("CheckStatus");
		case FTraceServerControl::EState::Command: return TEXT("Command");
		default: return TEXT("Unknown");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceServerControl::ChangeState(EState Expected, EState ChangeTo, uint32 Attempts)
{
	while (Attempts-- > 0)
	{
		const bool bChanged = State.compare_exchange_strong(Expected, ChangeTo);
		if (bChanged)
		{
			UE_LOGF(LogInsights, VeryVerbose, "Changing state from '%ls' -> '%ls'", LexState(Expected), LexState(ChangeTo))
			return true;
		}
		UE_LOGF(LogInsights, VeryVerbose, "Busy wait for '%ls'...", LexState(Expected));
		FPlatformProcess::Sleep(0.25);
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceServerControl::TriggerStatusUpdate()
{
	FGraphEventRef CheckStatusTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this]
	{
		if (bIsCancelRequested)
		{
			return;
		}
		FScopeLock _(&AsyncTaskLock);

		if (ChangeState(EState::Connected, EState::CheckStatus, GStateChangeRetries))
		{
			UpdateStatus();
			ChangeState(EState::CheckStatus, EState::Connected);
		}
	});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceServerControl::UpdateStatus()
{
	const UE::Trace::FStoreClient::FStatus* Status = Client->GetStatus();

	// Check current status
	const bool bServerIsRunning = Status != nullptr;
	bCanServerBeStarted.store(!bServerIsRunning, std::memory_order_relaxed);
	bCanServerBeStopped.store(bServerIsRunning, std::memory_order_relaxed);

	if (!bServerIsRunning)
	{
		ResetStatus();
		Client.Reset();
		ChangeState(EState::CheckStatus, EState::NotConnected);
		return;
	}

	TStringBuilder<64> PortsStringBuilder;
	if (Status)
	{
		bSponsored.store(Status->GetSponsored());
		PortsStringBuilder << TEXT("Recorder Port: ") << Status->GetRecorderPort()
							<< TEXT(", Store Port: ") << Status->GetStorePort();
	}

	// If not previously checked, also query version information
	if (StatusString.IsEmpty())
	{
		const UE::Trace::FStoreClient::FVersion* Version = Client->GetVersion();
		if (Version)
		{
			TStringBuilder<64> StatusStringBuilder;
			StatusStringBuilder << TEXT("Version: ") << Version->GetMajorVersion() << TEXT(".") << Version->GetMinorVersion();
			// Only print configuration if it's not release
			if (!Version->GetConfiguration().Equals(UTF8TEXT("Release")))
			{
				StatusStringBuilder << TEXT(" (") << Version->GetConfiguration() << TEXT(")");
			}
			StatusStringBuilder << TEXT(", ") << PortsStringBuilder;
			{
				FScopeLock Lock(&StringsLock);
				StatusString = StatusStringBuilder.ToString();
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceServerControl::ResetStatus()
{
	FScopeLock Lock(&StringsLock);
	StatusString.Empty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceServerControl::OnStart_Clicked()
{
	FGraphEventRef CommandTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this]
	{
		if (bIsCancelRequested)
		{
			return;
		}
		FScopeLock _(&AsyncTaskLock);

		if (ChangeState(EState::NotConnected, EState::Command, GStateChangeRetries))
		{
#if UE_TRACE_SERVER_CONTROLS_ENABLED
			FTraceServerControls::Start();
#endif
			ChangeState(EState::Command, EState::Connecting);

			if (!Client)
			{
				for (uint32 Attempts = 0; Attempts < GStartConnectAttempts; ++Attempts)
				{
					if (bIsCancelRequested)
					{
						break;
					}

					Client = UE::Trace::FStoreClient::Create(TEXT("127.0.0.1"));
					if (Client)
					{
						break;
					}
					FPlatformProcess::Sleep(GStartConnectFrequencySeconds);
				}
			}

			if (Client)
			{
				ChangeState(EState::Connecting, EState::Connected);
			}
			else
			{
				ChangeState(EState::Connecting, EState::NotConnected);
				UE_LOGF(LogInsights, Warning, "Failed to connect to store.");
			}
		}
		else
		{
			UE_LOGF(LogInsights, Warning, "Failed to start server.");
		}
	});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceServerControl::OnStop_Clicked()
{
	FGraphEventRef CommandTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this]
	{
		if (bIsCancelRequested)
		{
			return;
		}
		FScopeLock _(&AsyncTaskLock);

		if (ChangeState(EState::Connected, EState::Command, GStateChangeRetries))
		{
#if UE_TRACE_SERVER_CONTROLS_ENABLED
			FTraceServerControls::Stop();
#endif
			Client.Reset();
			ResetStatus();
			ChangeState(EState::Command, EState::NotConnected);
		}
		else
		{
			UE_LOGF(LogInsights, Warning, "Failed to stop server.");
		}
	});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceServerControl::OnSponsored_Changed()
{
	FGraphEventRef CommandTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this]
	{
		if (bIsCancelRequested)
		{
			return;
		}
		FScopeLock _(&AsyncTaskLock);

		if (ChangeState(EState::Connected, EState::Command, GStateChangeRetries))
		{
			const bool Success = Client->SetSponsored(!IsSponsored());
			ChangeState(EState::Command, EState::Connected);
			if (Success)
			{
				TriggerStatusUpdate();
			}
		}
		else
		{
			UE_LOGF(LogInsights, Warning, "Failed to set sponsored mode.");
		}
	});
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE