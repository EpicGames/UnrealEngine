// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationDriverCommon.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Widgets/SWindow.h"

// TraceAnalysis
#include "Trace/StoreConnection.h"

// TraceInsightsCore
#include "InsightsCore/Common/MiscUtils.h"

// TraceInsightsFrontend
#include "InsightsFrontend/ITraceInsightsFrontendModule.h"
#include "InsightsFrontend/Tests/TestUtils.h"
#include "InsightsFrontend/Widgets/STraceStoreWindow.h"

DECLARE_LOG_CATEGORY_EXTERN(MemoryInsightsTests, Log, All);

#if WITH_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FAutomationDriverUnrealInsightsHubMemoryInsightsTest, "System.Insights.Hub.MemoryInsights", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
FAutomationDriverPtr Driver;
TSharedPtr<SWindow> AutomationWindow;
END_DEFINE_SPEC(FAutomationDriverUnrealInsightsHubMemoryInsightsTest)
void FAutomationDriverUnrealInsightsHubMemoryInsightsTest::Define()
{
	BeforeEach([this]()
	{
		AutomationWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
		const FString AutomationWindowName = TEXT("Automation");
		if (AutomationWindow && AutomationWindow->GetTitle().ToString().Contains(AutomationWindowName))
		{
			AutomationWindow->Minimize();
		}

		ITraceInsightsFrontendModule& TraceInsightsFrontendModule = FModuleManager::LoadModuleChecked<ITraceInsightsFrontendModule>("TraceInsightsFrontend");

		if (IAutomationDriverModule::Get().IsEnabled())
		{
			IAutomationDriverModule::Get().Disable();
		}
		IAutomationDriverModule::Get().Enable();

		Driver = IAutomationDriverModule::Get().CreateDriver();
	});

	Describe("XMLReportsUpload", [this]()
	{
		It("should verify that user can upload xml reports in Memory Insights tab", EAsyncExecution::ThreadPool, FTimespan::FromSeconds(120), [this]()
		{
			UE::Insights::FTestUtils Utils(this);

			ITraceInsightsFrontendModule& TraceInsightsFrontendModule = FModuleManager::LoadModuleChecked<ITraceInsightsFrontendModule>("TraceInsightsFrontend");

			TSharedPtr<UE::Insights::STraceStoreWindow> TraceStoreWindow = TraceInsightsFrontendModule.GetTraceStoreWindow();
			if (!TraceStoreWindow.IsValid())
			{
				AddError("TraceStoreWindow should not be null");
				return;
			}
			if (!TraceStoreWindow->HasValidTraceStoreConnection())
			{
				AddError("TraceStoreWindow should be created");
				return;
			}
			UE::Trace::FStoreConnection& TraceStoreConnection = TraceStoreWindow->GetTraceStoreConnection();

			// Start tracing editor instance, not Lyra. There is no difference between them in this test.
			FString UEPath = FPlatformProcess::GenerateApplicationPath("UnrealEditor", EBuildConfiguration::Development);
			FString Parameters = TEXT("-trace=Bookmark,Memory -tracehost=127.0.0.1");
			constexpr bool bLaunchDetached = true;
			constexpr bool bLaunchHidden = false;
			constexpr bool bLaunchReallyHidden = false;
			uint32 ProcessID = 0;
			const int32 PriorityModifier = 0;
			const TCHAR* OptionalWorkingDirectory = nullptr;
			void* PipeWriteChild = nullptr;
			void* PipeReadChild = nullptr;
			FProcHandle EditorHandle = FPlatformProcess::CreateProc(*UEPath, *Parameters, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild);
			if (!EditorHandle.IsValid())
			{
				AddError("Editor should be started");
				return;
			}

			// Verify that LIVE trace appeared
			int Index = 0;
			auto TraceWaiter = [Driver = Driver, &Index](void) -> bool
			{
				auto Elements = Driver->FindElements(By::Id("TraceStatusColumnList"))->GetElements();
				for (int ElementIndex = 0; ElementIndex < Elements.Num(); ++ElementIndex)
				{
					if (Elements[ElementIndex]->GetText().ToString() == TEXT("LIVE"))
					{
						Index = ElementIndex;
						return true;
					}
				}
				return false;
			};

			if (!Driver->Wait(Until::Condition(TraceWaiter, FWaitTimeout::InSeconds(30))))
			{
				AddError("Live trace should appear");
				FPlatformProcess::TerminateProc(EditorHandle);
				return;
			}

			FDriverElementRef TraceElement = Driver->FindElements(By::Id("TraceList"))->GetElements()[Index];
			const FString TraceName = TraceElement->GetText().ToString();

			const FString StoreDir = TraceStoreConnection.GetStoreDir();
			const FString ProjectDir = FPaths::ProjectDir();
			const FString StoreTracePath = StoreDir / FString::Printf(TEXT("%s.utrace"), *TraceName);
			const FString StoreCachePath = StoreDir / FString::Printf(TEXT("%s.ucache"), *TraceName);
			const FString LogDirPath = ProjectDir / TEXT("TestResults");
			const FString TestLogPath = ProjectDir / TEXT("TestResults/Log.txt");
			const FString SuccessTestResult = TEXT("Test Completed. Result={Success}");

			// Test live trace
			FString TraceParameters = FString::Printf(TEXT("-InsightsTest -ABSLOG=\"%s\" -AutoQuit -ExecOnAnalysisCompleteCmd=\"Automation RunTests System.Insights.Trace.Analysis.MemoryInsights.UploadMemoryInsightsLLMXMLReportsTrace\" -OpenTraceFile=\"%s\""), *TestLogPath, *StoreTracePath);
			UE::Insights::FMiscUtils::OpenUnrealInsights(*TraceParameters);
			bool bLineFound = Utils.FileContainsString(TestLogPath, SuccessTestResult, 120.0f);
			TestTrue("Test for live trace should pass", bLineFound);

			IFileManager::Get().DeleteDirectory(*LogDirPath, false, true);
			FPlatformProcess::TerminateProc(EditorHandle);

			// Test stopped trace 
			UE::Insights::FMiscUtils::OpenUnrealInsights(*TraceParameters);
			bLineFound = Utils.FileContainsString(TestLogPath, SuccessTestResult, 120.0f);
			TestTrue("Test for stopped trace should pass", bLineFound);

			IFileManager::Get().DeleteDirectory(*LogDirPath, false, true);
			IFileManager::Get().Delete(*StoreTracePath);
			IFileManager::Get().Delete(*StoreCachePath);
		});
	});
	AfterEach([this]()
	{
		Driver.Reset();
		IAutomationDriverModule::Get().Disable();
		if (AutomationWindow)
		{
			AutomationWindow->Restore();
			AutomationWindow.Reset();
		}
	});
}

#endif // WITH_AUTOMATION_TESTS
