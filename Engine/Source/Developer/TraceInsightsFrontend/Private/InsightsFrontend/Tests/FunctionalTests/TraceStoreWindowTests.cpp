// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationDriverCommon.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"

// TraceAnalysis
#include "Trace/StoreConnection.h"

// TraceInsightsFrontend
#include "InsightsFrontend/ITraceInsightsFrontendModule.h"
#include "InsightsFrontend/Widgets/STraceStoreWindow.h"

DECLARE_LOG_CATEGORY_EXTERN(UnrealInsightsHubTests, Log, All);

#if WITH_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FAutomationDriverUnrealInsightsSessionBrowserTest, "System.Insights.Hub.SessionBrowser", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
FAutomationDriverPtr Driver;
TSharedPtr<SWindow> AutomationWindow;
END_DEFINE_SPEC(FAutomationDriverUnrealInsightsSessionBrowserTest)
void FAutomationDriverUnrealInsightsSessionBrowserTest::Define()
{
	BeforeEach([this]()
	{
		AutomationWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
		const FString AutomationWindowName = TEXT("Automation");
		if (AutomationWindow && AutomationWindow->GetTitle().ToString().Contains(AutomationWindowName))
		{
			AutomationWindow->Minimize();
		}

		ITraceInsightsFrontendModule& TraceInsightsFrontendModule = FModuleManager::LoadModuleChecked<ITraceInsightsFrontendModule>("TraceInsightsFrontend");

		if (IAutomationDriverModule::Get().IsEnabled())
		{
			IAutomationDriverModule::Get().Disable();
		}
		IAutomationDriverModule::Get().Enable();

		Driver = IAutomationDriverModule::Get().CreateDriver();
	});

	Describe("Trace Files", [this]()
	{
		It("should verify that user copy, rename and delete traces", EAsyncExecution::ThreadPool, [this]()
		{
			ITraceInsightsFrontendModule& TraceInsightsFrontendModule = FModuleManager::LoadModuleChecked<ITraceInsightsFrontendModule>("TraceInsightsFrontend");

			TSharedPtr<UE::Insights::STraceStoreWindow> TraceStoreWindow = TraceInsightsFrontendModule.GetTraceStoreWindow();
			TestTrue("TraceStoreWindow should not be null", TraceStoreWindow.IsValid());
			TestTrue("TraceStoreWindow should be created", TraceStoreWindow->HasValidTraceStoreConnection());

			UE::Trace::FStoreConnection& TraceStoreConnection = TraceStoreWindow->GetTraceStoreConnection();

			TraceStoreWindow->SetDeleteTraceConfirmationWindowVisibility(false);

			const FString StoreDir = TraceStoreConnection.GetStoreDir();
			const FString ProjectDir = FPaths::ProjectDir();

			const FString SourceTestTracePath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/Test.utrace");
			const FString SourceTestCachePath = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/Test.ucache");

			FString StoreTestTracePath = StoreDir / TEXT("Test.utrace");
			FString StoreTestCachePath = StoreDir / TEXT("Test.ucache");

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

			TestTrue("Trace in project exists", PlatformFile.FileExists(*SourceTestTracePath));
			TestTrue("Cache in project exists", PlatformFile.FileExists(*SourceTestCachePath));

			TestFalse("Trace in store should not exist before copy", PlatformFile.FileExists(*StoreTestTracePath));
			TestFalse("Cache in store should not exist before copy", PlatformFile.FileExists(*StoreTestCachePath));

			// Copy trace
			PlatformFile.CopyFile(*StoreTestTracePath, *SourceTestTracePath);
			PlatformFile.CopyFile(*StoreTestCachePath, *SourceTestCachePath);

			TestTrue("Trace copied", PlatformFile.FileExists(*StoreTestTracePath));
			TestTrue("Cache copied", PlatformFile.FileExists(*StoreTestCachePath));

			StoreTestTracePath = StoreDir / TEXT("TestUcacheRenaming.utrace");
			StoreTestCachePath = StoreDir / TEXT("TestUcacheRenaming.ucache");

			TestFalse("Renamed trace should not exist before renaming", PlatformFile.FileExists(*StoreTestTracePath));
			TestFalse("Renamed cache should not exist before renaming", PlatformFile.FileExists(*StoreTestCachePath));

			// Rename 
			int Index = 0;
			auto TraceWaiter = [Driver = Driver, &Index](void) -> bool
			{
				auto Elements = Driver->FindElements(By::Id("TraceList"))->GetElements();
				for (int i = 0; i < Elements.Num(); ++i)
				{
					if (Elements[i]->GetText().ToString() == TEXT("Test"))
					{
						Index = i;
						return true;
					}
				}
				return false;
			};

			bool bTestTraceExists = Driver->Wait(Until::Condition(TraceWaiter, FWaitTimeout::InSeconds(10)));
			if (!bTestTraceExists)
			{
				AddError("Trace should exists in Session Browser");
				return;
			}

			FDriverElementRef TraceElement = Driver->FindElements(By::Id("TraceList"))->GetElements()[Index];

			FDriverSequenceRef Sequence = Driver->CreateSequence();
			Sequence->Actions()
				.Click(TraceElement)
				.Type(EKeys::F2)
				.Type(TEXT("UcacheRenaming"))
				.Type(EKeys::Enter);

			TestTrue("Trace renamed", Sequence->Perform());

			TestTrue("Renamed trace should exists", PlatformFile.FileExists(*StoreTestTracePath));
			TestTrue("Renamed cache should exists", PlatformFile.FileExists(*StoreTestCachePath));

			// Delete
			FDriverElementRef OpenTraceButton = Driver->FindElement(By::Id("OpenTraceButton"));
			Driver->Wait(Until::ElementIsInteractable(OpenTraceButton, FWaitTimeout::InSeconds(10)));

			TraceElement = Driver->FindElements(By::Id("TraceList"))->GetElements()[0];
			TraceElement->Type(EKeys::Delete);

			TestFalse("Renamed trace should be deleted", PlatformFile.FileExists(*StoreTestTracePath));
			TestFalse("Renamed cache should be deleted", PlatformFile.FileExists(*StoreTestCachePath));
		});

		AfterEach([this]()
		{
			ITraceInsightsFrontendModule& TraceInsightsFrontendModule = FModuleManager::LoadModuleChecked<ITraceInsightsFrontendModule>("TraceInsightsFrontend");

			TSharedPtr<UE::Insights::STraceStoreWindow> TraceStoreWindow = TraceInsightsFrontendModule.GetTraceStoreWindow();
			if (TraceStoreWindow.IsValid() && TraceStoreWindow->HasValidTraceStoreConnection())
			{
				UE::Trace::FStoreConnection& TraceStoreConnection = TraceStoreWindow->GetTraceStoreConnection();

				const FString StoreDir = TraceStoreConnection.GetStoreDir();
				const FString StoreTestTracePath = StoreDir / TEXT("Test.utrace");
				const FString StoreTestCachePath = StoreDir / TEXT("Test.ucache");
				IFileManager::Get().Delete(*StoreTestTracePath, false, true);
				IFileManager::Get().Delete(*StoreTestCachePath, false, true);
			}
		});
	});



	AfterEach([this]()
	{
		Driver.Reset();
		IAutomationDriverModule::Get().Disable();
		if (AutomationWindow)
		{
			AutomationWindow->Restore();
			AutomationWindow.Reset();
		}
	});
}

#endif //WITH_AUTOMATION_TESTS