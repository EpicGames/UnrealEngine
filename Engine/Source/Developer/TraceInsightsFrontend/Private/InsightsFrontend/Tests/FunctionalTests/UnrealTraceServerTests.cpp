// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformFileManager.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"

// TraceAnalysis
#include "Trace/StoreConnection.h"

// TraceInsightsCore
#include "InsightsCore/Common/MiscUtils.h"

// TraceInsightsFrontend
#include "InsightsFrontend/ITraceInsightsFrontendModule.h"
#include "InsightsFrontend/Widgets/STraceStoreWindow.h"

#if WITH_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUnrealTraceServerStartingTest, "System.Insights.Hub.UnrealTraceServer.Starting", EAutomationTestFlags::ProgramContext | EAutomationTestFlags::EngineFilter)
bool FUnrealTraceServerStartingTest::RunTest(const FString& Parameters)
{
	const FString UnrealTraceServerName = TEXT("UnrealTraceServer");
	double Timeout = 10.0;

	auto WaitForUTSProcess = [](const FString& ProcessName, double Timeout, const bool bExpectedStatus) -> bool
	{
		double StartTime = FPlatformTime::Seconds();
		while (FPlatformTime::Seconds() - StartTime < Timeout)
		{
			if (FPlatformProcess::IsApplicationRunning(*ProcessName) == bExpectedStatus)
			{
				return bExpectedStatus;
			}
			FPlatformProcess::Sleep(0.1f);
		}
		return !bExpectedStatus;
	};

	auto WaitForConnectionIconStatus = [](UE::Trace::FStoreConnection& TraceStoreConnection, double Timeout, const bool bExpectedStatus) -> bool
	{
		double StartTime = FPlatformTime::Seconds();
		while (FPlatformTime::Seconds() - StartTime < Timeout)
		{
			bool bIsConnected = (TraceStoreConnection.GetStoreClient() != nullptr);
			if (bIsConnected == bExpectedStatus)
			{
				return bExpectedStatus;
			}
			FPlatformProcess::Sleep(0.1f);
		}
		return !bExpectedStatus;
	};

	ITraceInsightsFrontendModule& TraceInsightsFrontendModule = FModuleManager::LoadModuleChecked<ITraceInsightsFrontendModule>("TraceInsightsFrontend");

	TSharedPtr<UE::Insights::STraceStoreWindow> TraceStoreWindow = TraceInsightsFrontendModule.GetTraceStoreWindow();
	TestTrue("TraceStoreWindow should not be null", TraceStoreWindow.IsValid());
	TestTrue("TraceStoreWindow should be created", TraceStoreWindow->HasValidTraceStoreConnection());

	UE::Trace::FStoreConnection& TraceStoreConnection = TraceStoreWindow->GetTraceStoreConnection();

	bool bIsUnrealTraceServerRunning = WaitForUTSProcess(UnrealTraceServerName, Timeout, true);
	TestTrue("UnrealTraceServer should be in processes", bIsUnrealTraceServerRunning);

	bool bConnectionStatus = WaitForConnectionIconStatus(TraceStoreConnection, Timeout, true);
	TestTrue("Connection status should be true", bConnectionStatus);

#if PLATFORM_WINDOWS
	FString Command = FString::Printf(TEXT("/F /IM \"%s.exe\""), *UnrealTraceServerName);
	FPlatformProcess::ExecProcess(TEXT("taskkill"), *Command, nullptr, nullptr, nullptr);
#elif PLATFORM_MAC || PLATFORM_LINUX
	FString Command = FString::Printf(TEXT("killall \"%s\""), *UnrealTraceServerName);
	FPlatformProcess::ExecProcess(TEXT("/bin/sh"), *Command, nullptr, nullptr, nullptr);
#endif

	bIsUnrealTraceServerRunning = WaitForUTSProcess(UnrealTraceServerName, Timeout, false);
	TestFalse("UnrealTraceServer should not be in processes", bIsUnrealTraceServerRunning);

	bConnectionStatus = WaitForConnectionIconStatus(TraceStoreConnection, Timeout, true);
	TestTrue("Connection status should be true", bConnectionStatus);

	FString UEPath = FPlatformProcess::GenerateApplicationPath("UnrealEditor", EBuildConfiguration::Development);
	FString UEParameters = TEXT("");
	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = false;
	constexpr bool bLaunchReallyHidden = false;
	uint32 ProcessID = 0;
	const int32 PriorityModifier = 0;
	const TCHAR* OptionalWorkingDirectory = nullptr;
	void* PipeWriteChild = nullptr;
	void* PipeReadChild = nullptr;
	FProcHandle EditorHandle = FPlatformProcess::CreateProc(*UEPath, *UEParameters, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild);
	if (!EditorHandle.IsValid())
	{
		AddError("Editor should be started");
		return true;
	}

	bIsUnrealTraceServerRunning = WaitForUTSProcess(UnrealTraceServerName, Timeout, true);
	TestTrue("UnrealTraceServer should be in processes", bIsUnrealTraceServerRunning);

	bConnectionStatus = WaitForConnectionIconStatus(TraceStoreConnection, Timeout, true);
	TestTrue("Connection status should be true", bConnectionStatus);

	FPlatformProcess::TerminateProc(EditorHandle);

#if PLATFORM_WINDOWS
	FPlatformProcess::ExecProcess(TEXT("taskkill"), *Command, nullptr, nullptr, nullptr);
#elif PLATFORM_MAC || PLATFORM_LINUX
	FPlatformProcess::ExecProcess(TEXT("/bin/sh"), *Command, nullptr, nullptr, nullptr);
#endif

	bIsUnrealTraceServerRunning = WaitForUTSProcess(UnrealTraceServerName, Timeout, false);
	TestFalse("UnrealTraceServer should not be in processes", bIsUnrealTraceServerRunning);

	FString Path = FPaths::RootDir() / TEXT("EngineTest/SourceAssets/Utrace/r423_win64_game_10478456.utrace");
	UE::Insights::FMiscUtils::OpenUnrealInsights(*(FString::Printf(TEXT("-InsightsTest -AutoQuit -NoUI -OpenTraceFile=\"%s\""), *Path)));
	bIsUnrealTraceServerRunning = WaitForUTSProcess(UnrealTraceServerName, Timeout, true);
	TestTrue("UnrealTraceServer should be in processes", bIsUnrealTraceServerRunning);

	return true;
}

#endif // WITH_AUTOMATION_TESTS
