// Copyright Epic Games, Inc. All Rights Reserved.

#include "InsightsFrontend/Tests/TestUtils.h"

#include "HAL/PlatformTime.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"

#include "HAL/FileManagerGeneric.h"
#include "Misc/FileHelper.h"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

FTestUtils::FTestUtils(FAutomationTestBase* InTest) :
	Test(InTest)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTestUtils::FileContainsString(const FString& PathToFile, const FString& ExpectedString, double Timeout) const
{
	double StartTime = FPlatformTime::Seconds();
	while (FPlatformTime::Seconds() - StartTime < Timeout)
	{
		if (!FPaths::FileExists(PathToFile))
		{
			Test->AddInfo("Unable to find EngineTest.log at " + PathToFile);
			FPlatformProcess::Sleep(0.1f);
		}
		else
		{
			FString LogFileContents;
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			TUniquePtr<IFileHandle> FileHandle(PlatformFile.OpenRead(*PathToFile, true)); // Open the file with shared read access
			if (FileHandle)
			{
				TArray<uint8> FileData;
				FileData.SetNumUninitialized(static_cast<int32>(FileHandle->Size()));
				FileHandle->Read(FileData.GetData(), FileData.Num());
				FFileHelper::BufferToString(LogFileContents, FileData.GetData(), FileData.Num());

				if (LogFileContents.Contains(ExpectedString))
				{
					return true;
				}
			}
			FPlatformProcess::Sleep(0.1f);
		}
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
