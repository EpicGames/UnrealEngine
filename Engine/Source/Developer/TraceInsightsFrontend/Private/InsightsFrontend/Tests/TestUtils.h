// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "ProfilingDebugging/TraceAuxiliary.h" // for FTraceAuxiliary::EConnectionType

class FAutomationTestBase;

namespace UE::Insights
{

class FTestUtils
{
public:
	FTestUtils(FAutomationTestBase* Test);

	bool FileContainsString(const FString& PathToFile, const FString& ExpectedString, double Timeout) const;

private:
	FAutomationTestBase* Test;
};

} // namespace UE::Insights
