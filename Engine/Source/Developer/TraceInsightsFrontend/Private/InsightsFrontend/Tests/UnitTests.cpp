// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTraceInsightsFrontendUnitTest, "System.Insights.Frontend.UnitTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FTraceInsightsFrontendUnitTest::RunTest(const FString& Parameters)
{
	return true;
}
