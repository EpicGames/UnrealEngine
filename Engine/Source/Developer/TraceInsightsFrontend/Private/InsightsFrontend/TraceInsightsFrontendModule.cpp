// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceInsightsFrontendModule.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

// TraceAnalysis
#include "Trace/StoreConnection.h"

// TraceInsightsCore
#include "InsightsCore/Common/Log.h"
#include "InsightsCore/ITraceInsightsCoreModule.h"

// TraceInsightsFrontend
#include "InsightsFrontend/Common/InsightsFrontendStyle.h"
#include "InsightsFrontend/Common/InsightsAutomationController.h"
#include "InsightsFrontend/Common/Log.h"
#include "InsightsFrontend/Widgets/SConnectionWindow.h"
#include "InsightsFrontend/Widgets/STraceStoreWindow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::Frontend"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FInsightsFrontendTabs::TraceStoreTabId("TraceStore");
const FName FInsightsFrontendTabs::ConnectionTabId("Connection");

////////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FTraceInsightsFrontendModule, TraceInsightsFrontend);

FString FTraceInsightsFrontendModule::LayoutIni;

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::StartupModule()
{
	LLM_SCOPE_BYTAG(Insights);

	ITraceInsightsCoreModule& TraceInsightsCoreModule = FModuleManager::LoadModuleChecked<ITraceInsightsCoreModule>("TraceInsightsCore");

	FInsightsFrontendStyle::Initialize();
#if !WITH_EDITOR
	FAppStyle::SetAppStyleSet(FInsightsFrontendStyle::Get());
#endif

	RegisterTabSpawners();

	LayoutIni = GConfig->GetConfigFilename(TEXT("UnrealInsightsFrontendLayout"));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::ShutdownModule()
{
	if (PersistentLayout.IsValid())
	{
		// Save application layout.
		FLayoutSaveRestore::SaveToConfig(LayoutIni, PersistentLayout.ToSharedRef());
		GConfig->Flush(false, LayoutIni);
	}

	UnregisterTabSpawners();

	FInsightsFrontendStyle::Shutdown();

	TraceStoreConnection.Reset();

	InsightsAutomationController.Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FTraceInsightsFrontendModule::ConnectToStore(const TCHAR* InStoreHost, uint32 InStorePort)
{
	if (!TraceStoreConnection.IsValid())
	{
		TraceStoreConnection = MakeShared<UE::Trace::FStoreConnection>();
	}

	return TraceStoreConnection->ConnectToStore(InStoreHost, InStorePort);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::RegisterTabSpawners()
{
	using namespace UE::Insights;

	//const FInsightsMajorTabConfig& TraceStoreConfig = InsightsModule.FindMajorTabConfig(FInsightsFrontendTabs::TraceStoreTabId);
	//if (TraceStoreConfig.bIsAvailable)
	{
		// Register tab spawner for the Trace Store tab.
		FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FInsightsFrontendTabs::TraceStoreTabId,
			FOnSpawnTab::CreateRaw(this, &FTraceInsightsFrontendModule::SpawnTraceStoreTab))
			.SetDisplayName(/*TraceStoreConfig.TabLabel.IsSet() ? TraceStoreConfig.TabLabel.GetValue() :*/ LOCTEXT("TraceStoreTabTitle", "Trace Store"))
			.SetTooltipText(/*TraceStoreConfig.TabTooltip.IsSet() ? TraceStoreConfig.TabTooltip.GetValue() :*/ LOCTEXT("TraceStoreTooltipText", "Open the Trace Store Browser."))
			.SetIcon(/*TraceStoreConfig.TabIcon.IsSet() ? TraceStoreConfig.TabIcon.GetValue() :*/ FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.TraceStore"));

		TSharedRef<FWorkspaceItem> Group = /*TraceStoreConfig.WorkspaceGroup.IsValid() ?
			TraceStoreConfig.WorkspaceGroup.ToSharedRef() : */
			WorkspaceMenu::GetMenuStructure().GetDeveloperToolsProfilingCategory();
		TabSpawnerEntry.SetGroup(Group);
	}

	//const FInsightsMajorTabConfig& ConnectionConfig = InsightsModule.FindMajorTabConfig(FInsightsFrontendTabs::ConnectionTabId);
	//if (ConnectionConfig.bIsAvailable)
	{
		// Register tab spawner for the Connection tab.
		FTabSpawnerEntry& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FInsightsFrontendTabs::ConnectionTabId,
			FOnSpawnTab::CreateRaw(this, &FTraceInsightsFrontendModule::SpawnConnectionTab))
			.SetDisplayName(/*ConnectionConfig.TabLabel.IsSet() ? ConnectionConfig.TabLabel.GetValue() :*/ LOCTEXT("ConnectionTabTitle", "Connection"))
			.SetTooltipText(/*ConnectionConfig.TabTooltip.IsSet() ? ConnectionConfig.TabTooltip.GetValue() :*/ LOCTEXT("ConnectionTooltipText", "Open the Connection tab."))
			.SetIcon(/*ConnectionConfig.TabIcon.IsSet() ? ConnectionConfig.TabIcon.GetValue() : */ FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Connection"));

		TSharedRef<FWorkspaceItem> Group = /*ConnectionConfig.WorkspaceGroup.IsValid() ?
			ConnectionConfig.WorkspaceGroup.ToSharedRef() :*/
			WorkspaceMenu::GetMenuStructure().GetDeveloperToolsProfilingCategory();
		TabSpawnerEntry.SetGroup(Group);
	}

	InsightsAutomationController = MakeShared<FInsightsAutomationController>();

	InsightsAutomationController->Initialize();
	InsightsAutomationController->SetAutoQuit(CreateWindowParams.bAutoQuit);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::UnregisterTabSpawners()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FInsightsFrontendTabs::ConnectionTabId);
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FInsightsFrontendTabs::TraceStoreTabId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::CreateFrontendWindow(const FCreateFrontendWindowParams& InParams)
{
	CreateWindowParams = InParams;

	RegisterTabSpawners();

	//////////////////////////////////////////////////
	// Create the main window.

	const bool bEmbedTitleAreaContent = false;

	// Get desktop metrics. It also ensures the correct metrics will be used later in SWindow.
	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);
	const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(
		static_cast<float>(DisplayMetrics.PrimaryDisplayWorkAreaRect.Left),
		static_cast<float>(DisplayMetrics.PrimaryDisplayWorkAreaRect.Top));

	const FVector2D ClientSize(1280.0f * DPIScaleFactor, 720.0f * DPIScaleFactor);

	TSharedRef<SWindow> RootWindow = SNew(SWindow)
		.Title(NSLOCTEXT("TraceInsightsModule", "UnrealInsightsBrowserAppName", "Unreal Insights Frontend"))
		.CreateTitleBar(!bEmbedTitleAreaContent)
		.SupportsMaximize(true)
		.SupportsMinimize(true)
		.IsInitiallyMaximized(false)
		.IsInitiallyMinimized(false)
		.SizingRule(ESizingRule::UserSized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.ClientSize(ClientSize)
		.AdjustInitialSizeAndPositionForDPIScale(false);

	//RootWindow->GetTitleBar()->SetAllowMenuBar(true);

	const bool bShowRootWindowImmediately = false;
	FSlateApplication::Get().AddWindow(RootWindow, bShowRootWindowImmediately);

	FGlobalTabmanager::Get()->SetRootWindow(RootWindow);
	FGlobalTabmanager::Get()->SetAllowWindowMenuBar(true);

	FSlateNotificationManager::Get().SetRootWindow(RootWindow);

	//////////////////////////////////////////////////
	// Setup the window's content.

	TSharedRef<FTabManager::FLayout> DefaultLayout = FTabManager::NewLayout("UnrealInsightsFrontend_v1.0");
	DefaultLayout->AddArea
	(
		FTabManager::NewPrimaryArea()
		->Split
		(
			FTabManager::NewStack()
			->AddTab(FInsightsFrontendTabs::TraceStoreTabId, ETabState::OpenedTab)
			->AddTab(FInsightsFrontendTabs::ConnectionTabId, ETabState::OpenedTab)
			->AddTab(FName("SessionFrontend"), ETabState::OpenedTab)
			->SetForegroundTab(FInsightsFrontendTabs::TraceStoreTabId)
		)
	);

	// Create area and tab for Slate's WidgetReflector.
	DefaultLayout->AddArea
	(
		FTabManager::NewArea(800.0f * DPIScaleFactor, 400.0f * DPIScaleFactor)
		->SetWindow(FVector2D(10.0f * DPIScaleFactor, 10.0f * DPIScaleFactor), false)
		->Split
		(
			FTabManager::NewStack()->AddTab(FTabId("WidgetReflector"), InParams.bAllowDebugTools ? ETabState::OpenedTab : ETabState::ClosedTab)
		)
	);

	// Load layout from ini file.
	PersistentLayout = FLayoutSaveRestore::LoadFromConfig(LayoutIni, DefaultLayout);

	// Restore application layout.
	const EOutputCanBeNullptr OutputCanBeNullptr = EOutputCanBeNullptr::Never;
	TSharedPtr<SWidget> Content = FGlobalTabmanager::Get()->RestoreFrom(PersistentLayout.ToSharedRef(), RootWindow, bEmbedTitleAreaContent, OutputCanBeNullptr);

	RootWindow->SetContent(
		SNew(SOverlay)

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			Content.ToSharedRef()
		]

		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(0.0f, 15.0f, 4.0f, 0.0f)
		[
			VersionWidgetVM->CreateWidget()
		]
	);
	//RootWindow->GetOnWindowClosedEvent().AddRaw(this, &FTraceInsightsFrontendModule::OnWindowClosedEvent);

	//////////////////////////////////////////////////
	// Show the window.

	RootWindow->ShowWindow();
	const bool bForceWindowToFront = true;
	RootWindow->BringToFront(bForceWindowToFront);

	//////////////////////////////////////////////////
	// Set up command line parameter forwarding.

	TSharedPtr<STraceStoreWindow> TraceStoreWnd = GetTraceStoreWindow();
	if (TraceStoreWnd.IsValid())
	{
		TraceStoreWnd->SetEnableAutomaticTesting(InParams.bInitializeTesting);
		TraceStoreWnd->SetEnableDebugTools(InParams.bAllowDebugTools);
		TraceStoreWnd->SetStartProcessWithStompMalloc(InParams.bStartProcessWithStompMalloc);
		TraceStoreWnd->SetDisableFramerateThrottle(InParams.bDisableFramerateThrottle);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Special tab type, that cannot be dragged/undocked from the tab bar
 */
class SLockedTab : public SDockTab
{
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FTraceInsightsFrontendModule::SpawnTraceStoreTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SLockedTab)
		.TabRole(ETabRole::MajorTab);

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FTraceInsightsFrontendModule::OnTraceStoreTabClosed));

	if (!TraceStoreConnection.IsValid())
	{
		TraceStoreConnection = MakeShared<UE::Trace::FStoreConnection>();
	}

	TSharedRef<STraceStoreWindow> Window = SNew(STraceStoreWindow, TraceStoreConnection.ToSharedRef());
	DockTab->SetContent(Window);

	TraceStoreWindow = Window;

	if (!bIsMainTabSet)
	{
		FGlobalTabmanager::Get()->SetMainTab(DockTab);
		bIsMainTabSet = true;
	}

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::OnTraceStoreTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	TraceStoreWindow.Reset();

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SDockTab> FTraceInsightsFrontendModule::SpawnConnectionTab(const FSpawnTabArgs& Args)
{
	const TSharedRef<SDockTab> DockTab = SNew(SLockedTab)
		.TabRole(ETabRole::MajorTab)
		.CanEverClose(false)
		.OnCanCloseTab_Lambda([]() { return false; }); // can't close this tab

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FTraceInsightsFrontendModule::OnConnectionTabClosed));

	if (!TraceStoreConnection.IsValid())
	{
		TraceStoreConnection = MakeShared<UE::Trace::FStoreConnection>();
	}

	TSharedRef<SConnectionWindow> Window = SNew(SConnectionWindow, TraceStoreConnection.ToSharedRef());
	DockTab->SetContent(Window);

	ConnectionWindow = Window;

	return DockTab;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::OnConnectionTabClosed(TSharedRef<SDockTab> TabBeingClosed)
{
	ConnectionWindow.Reset();

	// Disable TabClosed delegate.
	TabBeingClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsFrontendSettings& FTraceInsightsFrontendModule::GetSettings()
{
	if (!Settings.IsValid())
	{
		Settings = MakeShared<FInsightsFrontendSettings>();
	}

	return *Settings;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceInsightsFrontendModule::RunAutomationTests(const FString& InCmd)
{
#if INSIGHTS_ENABLE_AUTOMATION

	FString ActualCmd = InCmd;
	ActualCmd.TrimCharInline(TEXT('\"'), nullptr);
	ActualCmd.TrimCharInline(TEXT('\''), nullptr);

	if (ActualCmd.StartsWith(TEXT("Automation RunTests")))
	{
		InsightsAutomationController->RunTests(InCmd);
	}
#else
	UE_LOGF(LogInsightsFrontend, Error, "Automated test could not execute because INSIGHTS_ENABLE_AUTOMATION is disabled.");
#endif // INSIGHTS_ENABLE_AUTOMATION
}

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE