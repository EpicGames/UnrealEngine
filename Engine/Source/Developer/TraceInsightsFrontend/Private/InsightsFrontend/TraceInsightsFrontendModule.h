// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Framework/Docking/TabManager.h"

// TraceInsightsCore
#include "InsightsCore/Common/MiscUtils.h"

// TraceInsightsFrontend
#include "InsightsFrontend/InsightsFrontendSettings.h"
#include "InsightsFrontend/ITraceInsightsFrontendModule.h"

namespace UE::Trace { class FStoreConnection; }

namespace UE::Insights
{

class FInsightsAutomationController;
class STraceStoreWindow;
class SConnectionWindow;

class FTraceInsightsFrontendModule : public ITraceInsightsFrontendModule
{
public:
	FTraceInsightsFrontendModule()
		: VersionWidgetVM(MakeShared<FVersionWidget>())
	{
	}
	virtual ~FTraceInsightsFrontendModule() = default;

	//////////////////////////////////////////////////
	// IModuleInterface

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	//////////////////////////////////////////////////
	// ITraceInsightsFrontendModule

	virtual bool ConnectToStore(const TCHAR* InStoreHost, uint32 InStorePort = 0) override;

	virtual void CreateFrontendWindow(const FCreateFrontendWindowParams& Params) override;

	virtual TSharedPtr<STraceStoreWindow> GetTraceStoreWindow() const override { return TraceStoreWindow.Pin(); }
	virtual TSharedPtr<SConnectionWindow> GetConnectionWindow() const override { return ConnectionWindow.Pin(); }

	virtual void RunAutomationTests(const FString& InCmd) override;

	//////////////////////////////////////////////////

	FInsightsFrontendSettings& GetSettings();

private:
	void RegisterTabSpawners();
	void UnregisterTabSpawners();

	/** Called to spawn the Trace Store major tab. */
	TSharedRef<SDockTab> SpawnTraceStoreTab(const FSpawnTabArgs& Args);

	/** Callback called when the Trace Store major tab is closed. */
	void OnTraceStoreTabClosed(TSharedRef<SDockTab> TabBeingClosed);

	/** Called to spawn the Connection major tab. */
	TSharedRef<SDockTab> SpawnConnectionTab(const FSpawnTabArgs& Args);

	/** Callback called when the Connection major tab is closed. */
	void OnConnectionTabClosed(TSharedRef<SDockTab> TabBeingClosed);

private:
	/** An instance of the main settings. */
	TSharedPtr<FInsightsFrontendSettings> Settings;

	FCreateFrontendWindowParams CreateWindowParams;

	TSharedPtr<FTabManager::FLayout> PersistentLayout;
	static FString LayoutIni;

	TSharedPtr<UE::Trace::FStoreConnection> TraceStoreConnection;

	/** A weak pointer to the Trace Store window. */
	TWeakPtr<STraceStoreWindow> TraceStoreWindow;

	/** A weak pointer to the Connection window. */
	TWeakPtr<SConnectionWindow> ConnectionWindow;

	bool bIsMainTabSet = false;

	TSharedPtr<UE::Insights::FInsightsAutomationController> InsightsAutomationController;

	/** The delegate to be invoked when this ticks. */
	FTickerDelegate OnTick;

	/** Handle to the registered OnTick. */
	FTSTicker::FDelegateHandle OnTickHandle;

	TSharedRef<FVersionWidget> VersionWidgetVM;
};

} // UE::Insights
