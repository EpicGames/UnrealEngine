// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceSetFilter.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

// TraceInsightsFrontend
#include "InsightsFrontend/ViewModels/TraceViewModel.h"
#include "InsightsFrontend/Widgets/STraceStoreWindow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::TraceSetFilter"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
// TTraceSetFilter
////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename TSetType>
void TTraceSetFilter<TSetType>::BuildMenu(FMenuBuilder& InMenuBuilder, STraceStoreWindow& Window)
{
	// Show/Hide All
	{
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda([this, &Window]()
			{
				if (FilterSet.IsEmpty())
				{
					FilterSet.Reset();
					for (const TSharedPtr<FTraceViewModel>& Trace : Window.GetAllAvailableTraces())
					{
						FilterSet.Add(GetFilterValueForTrace(*Trace));
					}
				}
				else
				{
					FilterSet.Reset();
				}
				Window.OnFilterChanged();
			});
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda([this]()
			{
				return FilterSet.IsEmpty() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			});

		InMenuBuilder.AddMenuEntry(
			ToggleAllActionLabel,
			ToggleAllActionTooltip.IsEmpty() ? TAttribute<FText>() : ToggleAllActionTooltip,
			FSlateIcon(),
			Action,
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

	InMenuBuilder.AddSeparator();

	TArray<TSetType> DefaultValues;
	AddDefaultValues(DefaultValues);

	TMap<TSetType, uint32> AllUniqueValues;
	for (const TSetType& Value : DefaultValues)
	{
		AllUniqueValues.Add(Value, 0);
	}
	for (const TSharedPtr<FTraceViewModel>& Trace : Window.GetAllAvailableTraces())
	{
		TSetType Value = GetFilterValueForTrace(*Trace);
		if (AllUniqueValues.Contains(Value))
		{
			AllUniqueValues[Value]++;
		}
		else
		{
			AllUniqueValues.Add(Value, 1);
		}
	}
	AllUniqueValues.KeySort([](const TSetType& A, const TSetType& B) { return A < B; });

	for (const auto& Pair : AllUniqueValues)
	{
		const TSetType& Value = Pair.Key;
		FUIAction Action;
		Action.ExecuteAction = FExecuteAction::CreateLambda([this, Value, &Window]()
			{
				if (FilterSet.Contains(Value))
				{
					FilterSet.Remove(Value);
				}
				else
				{
					FilterSet.Add(Value);
				}
				Window.OnFilterChanged();
			});
		Action.GetActionCheckState = FGetActionCheckState::CreateLambda([this, Value]()
			{
				return FilterSet.Contains(Value) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			});

		InMenuBuilder.AddMenuEntry(
			FText::Format(LOCTEXT("FilterValue_Fmt", "{0} ({1})"), ValueToText(Value), FText::AsNumber(Pair.Value)),
			TAttribute<FText>(), // no tooltip
			FSlateIcon(),
			Action,
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename TSetType>
TTraceSetFilter<TSetType>::TTraceSetFilter()
{
	ToggleAllActionLabel = LOCTEXT("ToggleAll_Label", "Show/Hide All");
	UndefinedValueLabel = LOCTEXT("UndefinedValueLabel", "N/A");
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FTraceFilterBy*
////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByPlatform::FTraceFilterByPlatform()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByPlatform_ToggleAll_Tooltip", "Shows or hides traces for all platforms.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByAppName::FTraceFilterByAppName()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByAppName_ToggleAll_Tooltip", "Shows or hides traces for all app names.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByBuildConfig::FTraceFilterByBuildConfig()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByBuildConfig_ToggleAll_Tooltip", "Shows or hides traces for all build configurations.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByBuildTarget::FTraceFilterByBuildTarget()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByBuildTarget_ToggleAll_Tooltip", "Shows or hides traces for all build targets.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByBuildVersion::FTraceFilterByBuildVersion()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByBuildVersion_ToggleAll_Tooltip", "Shows or hides all build versions.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByBranch::FTraceFilterByBranch()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByBranch_ToggleAll_Tooltip", "Shows or hides traces for all branches.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByChangelist::FTraceFilterByChangelist()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByChangelist_ToggleAll_Tooltip", "Shows or hides traces of all changelists.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceFilterByChangelist::AddDefaultValues(TArray<uint8>& InOutDefaultValues) const
{
	for (uint8 ChangelistCategory = 0; ChangelistCategory < (uint8)EChangelistCategory::InvalidOrMax; ++ChangelistCategory)
	{
		InOutDefaultValues.Add(ChangelistCategory);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTraceFilterByChangelist::ValueToText(const uint8 InValue) const
{
	switch ((EChangelistCategory)InValue)
	{
	case EChangelistCategory::Zero:    return LOCTEXT("FilterByChangelist_Zero", "0");
	case EChangelistCategory::NotZero: return LOCTEXT("FilterByChangelist_NotZero", "!= 0");
	default:                           return UndefinedValueLabel;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByEngineVersion::FTraceFilterByEngineVersion()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByEngineVersion_ToggleAll_Tooltip", "Shows or hides all engine versions.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterBySize::FTraceFilterBySize()
{
	ToggleAllActionTooltip = LOCTEXT("FilterBySize_ToggleAll_Tooltip", "Shows or hides traces of all sizes.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceFilterBySize::AddDefaultValues(TArray<uint8>& InOutDefaultValues) const
{
	for (uint8 SizeCategory = 0; SizeCategory < (uint8)ESizeCategory::InvalidOrMax; ++SizeCategory)
	{
		InOutDefaultValues.Add(SizeCategory);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTraceFilterBySize::ValueToText(const uint8 InValue) const
{
	switch ((ESizeCategory)InValue)
	{
		case ESizeCategory::Empty:  return LOCTEXT("FilterBySize_Empty",  "Empty (0 bytes)");
		case ESizeCategory::Small:  return LOCTEXT("FilterBySize_Small",  "Small (< 1 MiB)");
		case ESizeCategory::Medium: return LOCTEXT("FilterBySize_Medium", "Medium (< 1 GiB)");
		case ESizeCategory::Large:  return LOCTEXT("FilterBySize_Large",  "Large (\u2265 1 GiB)");
		default:                    return UndefinedValueLabel;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FTraceFilterByStatus::FTraceFilterByStatus()
{
	ToggleAllActionTooltip = LOCTEXT("FilterByStatus_ToggleAll_Tooltip", "Shows or hides all traces.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FTraceFilterByStatus::AddDefaultValues(TArray<bool>& InOutDefaultValues) const
{
	InOutDefaultValues.Add(false);
	InOutDefaultValues.Add(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText FTraceFilterByStatus::ValueToText(const bool InValue) const
{
	if (InValue)
	{
		return LOCTEXT("FilterByStatus_Live", "LIVE");
	}
	else
	{
		return LOCTEXT("FilterByStatus_Offline", "Offline");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
