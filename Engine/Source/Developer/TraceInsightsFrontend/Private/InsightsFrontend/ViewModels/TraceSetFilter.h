// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/IFilter.h"
#include "Templates/SharedPointer.h"

// TraceInsightsFrontend
#include "InsightsFrontend/ViewModels/TraceViewModel.h"

class FMenuBuilder;

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

template<typename TSetType>
class TTraceSetFilter : public IFilter<const FTraceViewModel&>, public TSharedFromThis< TTraceSetFilter<TSetType> >
{
public:
	TTraceSetFilter();
	virtual ~TTraceSetFilter() = default;

	typedef const FTraceViewModel& ItemType;

	/** Broadcasts anytime the restrictions of the Filter changes. */
	DECLARE_DERIVED_EVENT(TTraceSetFilter, IFilter<ItemType>::FChangedEvent, FChangedEvent);
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

	/** Returns whether the specified Trace passes the Filter's restrictions. */
	virtual bool PassesFilter(const FTraceViewModel& InTrace) const override
	{
		return FilterSet.IsEmpty() || !FilterSet.Contains(GetFilterValueForTrace(InTrace));
	}

	bool IsEmpty() const { return FilterSet.IsEmpty(); }

	void Reset() { FilterSet.Reset(); }

	virtual void BuildMenu(FMenuBuilder& InMenuBuilder, class STraceStoreWindow& InWindow);

protected:
	virtual void AddDefaultValues(TArray<TSetType>& InOutDefaultValues) const { }
	virtual TSetType GetFilterValueForTrace(const FTraceViewModel& InTrace) const = 0;
	virtual FText ValueToText(const TSetType Value) const = 0;

protected:
	/**	The event that fires whenever new search terms are provided */
	FChangedEvent ChangedEvent;

	/** The set of values used to filter */
	TSet<TSetType> FilterSet;

	FText ToggleAllActionLabel;
	FText ToggleAllActionTooltip;
	FText UndefinedValueLabel;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByStringSet : public TTraceSetFilter<FString>
{
protected:
	virtual FText ValueToText(const FString InValue) const override
	{
		return InValue.IsEmpty() ? UndefinedValueLabel : FText::FromString(InValue);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByPlatform : public FTraceFilterByStringSet
{
public:
	FTraceFilterByPlatform();

protected:
	virtual FString GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return InTrace.GetPlatform();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByAppName : public FTraceFilterByStringSet
{
public:
	FTraceFilterByAppName();

protected:
	virtual FString GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return InTrace.GetAppName();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByBuildConfig : public TTraceSetFilter<uint8>
{
public:
	FTraceFilterByBuildConfig();

protected:
	virtual uint8 GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return (uint8)InTrace.GetBuildConfiguration();
	}

	virtual FText ValueToText(const uint8 InValue) const override
	{
		const TCHAR* Str = LexToString((EBuildConfiguration)InValue);
		return Str ? FText::FromString(Str) : UndefinedValueLabel;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByBuildTarget : public TTraceSetFilter<uint8>
{
public:
	FTraceFilterByBuildTarget();

protected:
	virtual uint8 GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return (uint8)InTrace.GetBuildTarget();
	}

	virtual FText ValueToText(const uint8 InValue) const override
	{
		const TCHAR* Str = LexToString((EBuildTargetType)InValue);
		return Str ? FText::FromString(Str) : UndefinedValueLabel;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByBuildVersion : public FTraceFilterByStringSet
{
public:
	FTraceFilterByBuildVersion();

protected:
	virtual FString GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return InTrace.GetBuildVersion();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByBranch : public FTraceFilterByStringSet
{
public:
	FTraceFilterByBranch();

protected:
	virtual FString GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return InTrace.GetBuildBranch();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByChangelist : public TTraceSetFilter<uint8>
{
public:
	enum class EChangelistCategory : uint8
	{
		Zero,  // CL 0
		NotZero,

		InvalidOrMax
	};

public:
	FTraceFilterByChangelist();

protected:
	virtual void AddDefaultValues(TArray<uint8>& InOutDefaultValues) const override;

	virtual uint8 GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		if (InTrace.GetChangelist() == 0)
		{
			return (uint8)EChangelistCategory::Zero;
		}
		else
		{
			return (uint8)EChangelistCategory::NotZero;
		}
	}

	virtual FText ValueToText(const uint8 InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByEngineVersion : public FTraceFilterByStringSet
{
public:
	FTraceFilterByEngineVersion();

protected:
	virtual FString GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return InTrace.GetEngineVersion();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterBySize : public TTraceSetFilter<uint8>
{
public:
	enum class ESizeCategory : uint8
	{
		Empty,  // 0 bytes
		Small,  // < 1 MiB
		Medium, // < 1 GiB
		Large,  // >= 1 GiB


		InvalidOrMax
	};

public:
	FTraceFilterBySize();

protected:
	virtual void AddDefaultValues(TArray<uint8>& InOutDefaultValues) const override;

	virtual uint8 GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		const uint64 TraceSize = InTrace.GetSize();
		if (TraceSize == 0)
		{
			return (uint8)ESizeCategory::Empty;
		}
		else if (TraceSize < 1024ull * 1024ull)
		{
			return (uint8)ESizeCategory::Small;
		}
		else if (TraceSize < 1024ull * 1024ull * 1024ull)
		{
			return (uint8)ESizeCategory::Medium;
		}
		else
		{
			return (uint8)ESizeCategory::Large;
		}
	}

	virtual FText ValueToText(const uint8 InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FTraceFilterByStatus : public TTraceSetFilter<bool>
{
public:
	FTraceFilterByStatus();

protected:
	virtual void AddDefaultValues(TArray<bool>& InOutDefaultValues) const override;

	virtual bool GetFilterValueForTrace(const FTraceViewModel& InTrace) const override
	{
		return InTrace.IsLive();
	}

	virtual FText ValueToText(const bool InValue) const override;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
