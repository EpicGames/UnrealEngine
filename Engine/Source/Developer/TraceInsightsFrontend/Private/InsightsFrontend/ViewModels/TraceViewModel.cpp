// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceViewModel.h"

#include "Containers/UnrealString.h"

namespace UE::Insights
{

FText FTraceViewModel::AnsiStringViewToText(const FAnsiStringView& AnsiStringView)
{
	FString FatString = FString::ConstructFromPtrSize(AnsiStringView.GetData(), AnsiStringView.Len());
	return FText::FromString(FatString);
}

void FTraceViewModel::SetEngineVersion(const FString& InEngineVersion)
{
	FString Version = InEngineVersion;

	if (Version.IsEmpty())
	{
		FString BranchString = BranchText.ToString();
		if (!BranchString.IsEmpty())
		{
			FStringView UE4ReleasePrefix = TEXTVIEW("++UE4+Release-");
			FStringView UE5ReleasePrefix = TEXTVIEW("++UE5+Release-");
			if (BranchString.StartsWith(UE5ReleasePrefix))
			{
				Version = BranchString.RightChop(UE5ReleasePrefix.Len());
			}
			else if (BranchString.StartsWith(TEXTVIEW("++UE5+")) ||
					 BranchString.StartsWith(TEXTVIEW("UE5")))
			{
				Version = "5";
			}
			else if (BranchString.StartsWith(UE4ReleasePrefix))
			{
				Version = BranchString.RightChop(UE4ReleasePrefix.Len());
			}
			else if (BranchString.StartsWith(TEXTVIEW("++UE4+")) ||
					 BranchString.StartsWith(TEXTVIEW("UE4")))
			{
				Version = "4";
			}
		}
	}
	else if (!BuildVersionText.IsEmpty())
	{
		// Trim the BuildVersion from the EngineVersion, if it is included and it is the same.
		int32 Index = Version.Find(BuildVersionText.ToString(), ESearchCase::CaseSensitive);
		if (Index > 0 && Version[Index - 1] == TEXT('-'))
		{
			Version.LeftInline(Index - 1);
		}
	}

	EngineVersionText = FText::FromString(Version);

	auto ReadVersionNumber = [](FString& InOutVersion) -> uint64
		{
			if (InOutVersion.IsEmpty())
			{
				return 0ull;
			}
			int32 Index = 0;
			while (Index < InOutVersion.Len() &&
				InOutVersion[Index] >= TEXT('0') &&
				InOutVersion[Index] <= TEXT('9'))
			{
				++Index;
			}
			FString VersionNumberString = InOutVersion.Left(Index);
			int32 VersionNumber = FCString::Atoi(*VersionNumberString);
			if (Index < InOutVersion.Len() && InOutVersion[Index] == TEXT('.'))
			{
				InOutVersion.RightChopInline(Index + 1);
			}
			else
			{
				InOutVersion.Reset();
			}
			return uint64(FMath::Max(0, VersionNumber)) + 1; // +1 is to sort "5.0" after "5"
		};

	uint64 Major = ReadVersionNumber(Version);
	uint64 Minor = ReadVersionNumber(Version);
	uint64 Patch = ReadVersionNumber(Version);

	EngineVersionNumber = (Major << 54) | (Minor << 44) | (Patch << 32) | uint64(Changelist);
}

void FTraceViewModel::SetCommandLine(const FString& InCommandLine)
{
	CommandLineText = FText::FromString(InCommandLine.TrimStartAndEnd());
}

void FTraceViewModel::SetVFSPaths(const FString& InVFSPaths)
{
	TArray<FString> VFSPathsArray;
	InVFSPaths.ParseIntoArray(VFSPathsArray, TEXT(";"), true);
	int32 NumVFSPaths = 0;
	TStringBuilder<1024> VFSPathsBuilder;
	for (FString& Path : VFSPathsArray)
	{
		Path.TrimStartAndEndInline();
		if (!Path.IsEmpty())
		{
			if (NumVFSPaths & 1)
			{
				VFSPathsBuilder.Append(TEXT(" \u2192 "));
			}
			else if (NumVFSPaths > 0)
			{
				VFSPathsBuilder.Append(TEXT("\n"));
			}
			++NumVFSPaths;
			VFSPathsBuilder.Append(Path);
		}
	}
	VFSPathsText = FText::FromString(VFSPathsBuilder.ToString());
}

} // namespace UE::Insights
