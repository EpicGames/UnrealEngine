// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/StringFwd.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"

class SEditableTextBox;

namespace UE::Insights
{

class FTraceViewModel
{
public:
	static constexpr uint32 InvalidTraceId = 0;

public:
	FTraceViewModel() = default;

	uint64 GetChangeSerial() const { return ChangeSerial; }
	void SetChangeSerial(uint64 InChangeSerial) { ChangeSerial = InChangeSerial; }

	uint32 GetId() const { return TraceId; }
	bool HasValidId() const { return TraceId != InvalidTraceId; }
	void SetId(uint32 InId) { TraceId = InId; }

	uint64 GetSize() const { return Size; }
	void SetSize(uint64 InSize) { Size = InSize; }

	FDateTime GetTimestamp() const { return Timestamp; }
	void SetTimestamp(const FDateTime& InTimestamp) { Timestamp = InTimestamp; }

	const FText& GetNameText() const { return NameText; }
	const FString& GetName() const { return NameText.ToString(); }
	void SetName(const FString& InName) { NameText = FText::FromString(InName); }
	bool HasSameName(const FTraceViewModel& Other) const { return NameText.EqualTo(Other.GetNameText()); }

	const FText& GetUriText() const { return UriText; }
	const FString& GetUri() const { return UriText.ToString(); }
	void SetUri(const FString& InUri) { UriText = FText::FromString(InUri); }
	bool HasSameUri(const FTraceViewModel& Other) const { return UriText.EqualTo(Other.GetUriText()); }

	const FText& GetPlatformText() const { return PlatformText; }
	const FString& GetPlatform() const { return PlatformText.ToString(); }
	void SetPlatform(const FString& InPlatform) { PlatformText = FText::FromString(InPlatform); }

	const FText& GetAppNameText() const { return AppNameText; }
	const FString& GetAppName() const { return AppNameText.ToString(); }
	void SetAppName(const FString& InAppName) { AppNameText = FText::FromString(InAppName); }

	EBuildConfiguration GetBuildConfiguration() const { return ConfigurationType; }
	void SetBuildConfiguration(EBuildConfiguration InBuildConfiguration) { ConfigurationType = InBuildConfiguration; }

	EBuildTargetType GetBuildTarget() const { return TargetType; }
	void SetBuildTarget(EBuildTargetType InBuildTarget) { TargetType = InBuildTarget; }

	const FText& GetBuildVersionText() const { return BuildVersionText; }
	const FString& GetBuildVersion() const { return BuildVersionText.ToString(); }
	void SetBuildVersion(const FString& InBuildVersion) { BuildVersionText = FText::FromString(InBuildVersion); }

	const FText& GetBuildBranchText() const { return BranchText; }
	const FString& GetBuildBranch() const { return BranchText.ToString(); }
	void SetBuildBranch(const FString& InBuildBranch) { BranchText = FText::FromString(InBuildBranch); }

	uint32 GetChangelist() const { return Changelist; }
	void SetChangelist(uint32 InChangelist) { Changelist = InChangelist; }

	const FText& GetEngineVersionText() const { return EngineVersionText; }
	const FString& GetEngineVersion() const { return EngineVersionText.ToString(); }
	uint64 GetEngineVersionNumber() const { return EngineVersionNumber; }
	void SetEngineVersion(const FString& InEngineVersion);

	const FText& GetCommandLineText() const { return CommandLineText; }
	const FString& GetCommandLine() const { return CommandLineText.ToString(); }
	void SetCommandLine(const FString& InCommandLine);

	const FText& GetVFSPathsText() const { return VFSPathsText; }
	const FString& GetVFSPaths() const { return VFSPathsText.ToString(); }
	void SetVFSPaths(const FString& InVFSPaths);

	bool IsLive() const { return bIsLive; }
	void SetIsLive(bool bOnOff) { bIsLive = bOnOff; }

	bool IsMetadataUpdated() const { return bIsMetadataUpdated; }
	void SetIsMetadataUpdated(bool bOnOff) { bIsMetadataUpdated = bOnOff; }

	bool IsRenaming() const { return bIsRenaming; }
	void StartRenaming() { bIsRenaming = true; }
	void StopRenaming() { bIsRenaming = false; }
	TSharedPtr<SEditableTextBox> GetRenameTextBox() const& { return RenameTextBox.Pin(); }
	void SetRenameTextBox(TWeakPtr<SEditableTextBox> InRenameTextBox) { RenameTextBox = InRenameTextBox; }

	const FSlateColor& GetDirectoryColor() const { return DirectoryColor; }
	void SetDirectoryColor(const FSlateColor& InColor) { DirectoryColor = InColor; }

	const uint32 GetIpAddress() const { return IpAddress; }
	void SetIpAddress(uint32 InIpAddress) { IpAddress = InIpAddress; }

	static FDateTime ConvertTimestamp(uint64 InTimestamp)
	{
		return FDateTime(static_cast<int64>(InTimestamp));
	}

	static FText AnsiStringViewToText(const FAnsiStringView& AnsiStringView);

private:
	uint64 ChangeSerial = 0;
	uint64 EngineVersionNumber = 0; // only used for sorting
	uint32 TraceId = InvalidTraceId;
	uint32 Changelist = 0;
	uint64 Size = 0;
	FDateTime Timestamp = 0;

	FText NameText;
	FText UriText;
	FText PlatformText;
	FText AppNameText;
	FText BuildVersionText;
	FText BranchText;
	FText EngineVersionText;
	FText CommandLineText;
	FText VFSPathsText;

	FSlateColor DirectoryColor;

	uint32 IpAddress = 0;

	EBuildConfiguration ConfigurationType = EBuildConfiguration::Unknown;
	EBuildTargetType TargetType = EBuildTargetType::Unknown;

	bool bIsLive = false;
	bool bIsMetadataUpdated = false;
	bool bIsRenaming = false;

	TWeakPtr<SEditableTextBox> RenameTextBox;
};

} // namespace UE::Insights
