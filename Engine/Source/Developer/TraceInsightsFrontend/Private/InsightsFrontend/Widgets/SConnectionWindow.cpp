// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConnectionWindow.h"

#include "Async/TaskGraphInterfaces.h"
#include "Internationalization/Internationalization.h"
#include "SlateOptMacros.h"
#include "SocketSubsystem.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

// TraceAnalysis
#include "Trace/ControlClient.h"
#include "Trace/StoreConnection.h"

// TraceInsightsCore
#include "InsightsCore/Common/MiscUtils.h"

// TraceInsightsFrontend
#include "InsightsFrontend/Common/InsightsFrontendStyle.h"
#include "InsightsFrontend/Common/Log.h"

#define LOCTEXT_NAMESPACE "UE::Insights::SConnectionWindow"

namespace UE::Insights
{

// Should match GDefaultChannels (from Runtime\Core\Private\ProfilingDebugging\TraceAuxiliary.cpp).
// We cannot use GDefaultChannels directly as UE_TRACE_ENABLED may be off.
static const TCHAR* GInsightsDefaultChannelPreset = TEXT("cpu,gpu,frame,log,bookmark,screenshot,region");

////////////////////////////////////////////////////////////////////////////////////////////////////

SConnectionWindow::SConnectionWindow()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

SConnectionWindow::~SConnectionWindow()
{
	if (ConnectTask && !ConnectTask->IsComplete())
	{
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(ConnectTask);
		ConnectTask = nullptr;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SConnectionWindow::Construct(const FArguments& InArgs, TSharedRef<UE::Trace::FStoreConnection> InTraceStoreConnection)
{
	TraceStoreConnection = InTraceStoreConnection;

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(0.0f)
				.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(FSlateColor(EStyleColor::Panel))
			]
		]

		// Overlay slot for the main window area
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoHeight()
			.Padding(3.0f, 3.0f)
			[
				ConstructConnectPanel()
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SConnectionWindow::ConstructConnectPanel()
{
	FText InitialChannelsExampleText = FText::FromString(FString::Printf(TEXT("default,counter,stats,file,loadtime,assetloadtime,task\ndefault=%s"), GInsightsDefaultChannelPreset));

	TSharedRef<SWidget> Widget = SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 12.0f, 12.0f, 0.0f)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("Font.Large.Bold"))
			.Text(LOCTEXT("DirectTrace_Title", "Direct Trace"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 4.0f, 12.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DirectTrace_Subtitle", "Start Unreal Insights in \"listen for a Direct Trace stream\" mode..."))
			.IsEnabled(false)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 12.0f, 12.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(200.0f)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DirectTraceCmdLineParams_Text", "Direct Trace CmdLine Params"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.IsEnabled(false)
				.IsChecked(ECheckBoxState::Checked)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ListenForDirectTrace_Text", "-ListenForDirectTrace"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.IsEnabled(false)
				.IsChecked(ECheckBoxState::Checked)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DirectTracePort_Text", "-DirectTracePort="))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(DirectTracePortTextBox, SEditableTextBox)
				.MinDesiredWidth(60.0f)
				.RevertTextOnEscape(true)
				.SelectAllTextWhenFocused(true)
				.ToolTipText(LOCTEXT("DirectTracePort_Tooltip", "Port to be used by Unreal Insights to listen for a direct trace stream."))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 8.0f, 12.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(200.0f)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MiscCmdLineParams_Text", "Misc CmdLine Params"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(AutoQuitCheckBox, SCheckBox)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AutoQuit_Text", "-AutoQuit"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(WaitForSymbolResolverCheckBox, SCheckBox)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("WaitForSymbolResolver_Text", "-WaitForSymbolResolver"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(DisableFramerateThrottleCheckBox, SCheckBox)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DisableFramerateThrottle_Text", "-DisableFramerateThrottle"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(6.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(InsightsTestCheckBox, SCheckBox)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("InsightsTest_Text", "-InsightsTest"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(NoUICheckBox, SCheckBox)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoUI_Text", "-NoUI"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(DebugToolsCheckBox, SCheckBox)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DebugTools_Text", "-DebugTools"))
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 8.0f, 12.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(200.0f)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AdditionalCmdLineParams_Text", "Additional CmdLine Params"))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(120.0f)
				[
					SAssignNew(DirectTraceAdditionalParamsTextBox, SEditableTextBox)
					.RevertTextOnEscape(true)
					.SelectAllTextWhenFocused(true)
					.ToolTipText(LOCTEXT("DirectTraceAdditionalParams_Tooltip", "Additional command line parameter for launching Unreal Insights (Direct Trace mode)."))
					.MinDesiredWidth(40.0f)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 12.0f, 12.0f, 0.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
				.Text(LOCTEXT("StartUnrealInsights", "Start Unreal Insights"))
				.ToolTipText(LOCTEXT("StartUnrealInsights_ToolTip", "Start Unreal Insights in \"listen for a Direct Trace stream\" mode."))
				.OnClicked(this, &SConnectionWindow::StartUnrealInsightsDirectTrace_OnClicked)
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 12.0f, 12.0f, 0.0f)
		[
			SNew(SSeparator)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 12.0f, 12.0f, 0.0f)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("Font.Large.Bold"))
			.Text(LOCTEXT("Connection_Title", "Connection"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 4.0f, 12.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Connection_Subtitle", "Late connect the trace source (game client, server, editor or other UE program) with the trace recorder/processor (the destination of trace data)..."))
			.IsEnabled(false)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 8.0f, 12.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(200.0f)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TraceSource_Text", "Trace Source"))
					.ToolTipText(LOCTEXT("TraceSource_ToolTip", "Source of the trace stream (i.e. the trace emitter)\n\nThe IP address of the machine/console running the editor, game client/server or any UE application having trace enabled."))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(120.0f)
				[
					SAssignNew(RunningInstanceAddressTextBox, SEditableTextBox)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 12.0f, 12.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(200.0f)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TraceDestination_Text", "Trace Destination"))
					.ToolTipText(LOCTEXT("TraceDestination_ToolTip", "Destination of the trace stream (i.e. the trace recorder/processor)\n\nThe IP address of the machine running UnrealTraceServer or\nthe IP address of the machine running Unreal Insights in Direct Trace mode."))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(120.0f)
				[
					SAssignNew(TraceRecorderAddressTextBox, SEditableTextBox)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 8.0f, 12.0f, 0.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(200.0f)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("InitialChannelsText", "Initial Channels"))
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.MinDesiredWidth(120.0f)
				[
					SAssignNew(ChannelsTextBox, SEditableTextBox)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(220.0f, 4.0f, 12.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("InitialChannelsNoteText", "Comma-separated list of channels/presets to enable when connected."))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(220.0f, 2.0f, 12.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Top)
			.Padding(FMargin(0.0f, 4.0f, 0.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("InitialChannelsExamplesTitle", "Examples"))
			]
			+ SHorizontalBox::Slot()
			.Padding(6.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.Text(InitialChannelsExampleText)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(220.0f, 2.0f, 12.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("InitialChannelsNote2Text", "Some channels/presets (like \"memory\") cannot be enabled on late connections."))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(12.0f, 8.0f, 12.0f, 12.0f)
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
				.Text(LOCTEXT("Connect", "Connect"))
				.ToolTipText(LOCTEXT("ConnectToolTip", "Late connect the trace source (game client, server, editor or other UE program)\nwith the trace recorder/processor (the destination of trace data)."))
				.OnClicked(this, &SConnectionWindow::Connect_OnClicked)
				.IsEnabled_Lambda([this]() { return !bIsConnecting; })
			]
		]

		// Notification area overlay
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(16.0f)
		[
			SAssignNew(NotificationList, SNotificationList)
		];

	// Set the default port for direct trace.
	DirectTracePortTextBox->SetText(FText::FromString(TEXT("1986")));

	const FText LocalHost = FText::FromString(TEXT("127.0.0.1"));

	if (TraceStoreConnection->IsLocalHost())
	{
		TSharedPtr<FInternetAddr> RecorderAddr;
		if (ISocketSubsystem* Sockets = ISocketSubsystem::Get())
		{
			bool bCanBindAll = false;
			RecorderAddr = Sockets->GetLocalHostAddr(*GLog, bCanBindAll);
		}
		if (RecorderAddr.IsValid())
		{
			const FString RecorderAddrStr = RecorderAddr->ToString(false);
			TraceRecorderAddressTextBox->SetText(FText::FromString(RecorderAddrStr));
		}
		else
		{
			TraceRecorderAddressTextBox->SetText(LocalHost);
		}
	}
	else // remote trace store
	{
		uint32 StoreAddress = 0;
		uint32 StorePort = 0;
		if (TraceStoreConnection->GetStoreAddressAndPort(StoreAddress, StorePort))
		{
			const FString RecorderAddrStr = FString::Printf(TEXT("%u.%u.%u.%u"), (StoreAddress >> 24) & 0xFF, (StoreAddress >> 16) & 0xFF, (StoreAddress >> 8) & 0xFF, StoreAddress & 0xFF);
			TraceRecorderAddressTextBox->SetText(FText::FromString(RecorderAddrStr));
		}
		else
		{
			TraceRecorderAddressTextBox->SetText(FText::FromString(TraceStoreConnection->GetLastStoreHost()));
		}
	}

	RunningInstanceAddressTextBox->SetText(LocalHost);
	ChannelsTextBox->SetText(FText::FromStringView(TEXT("default")));

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SConnectionWindow::StartUnrealInsightsDirectTrace_OnClicked()
{
	if (!DirectTracePortTextBox.IsValid())
	{
		return FReply::Handled();
	}

	const uint32 Port = FCString::Atoi(*DirectTracePortTextBox->GetText().ToString());

	FString CmdLine = FString::Printf(TEXT("-ListenForDirectTrace -DirectTracePort=%u"), Port);

	if (AutoQuitCheckBox->IsChecked())
	{
		CmdLine.Append(TEXT(" -AutoQuit"));
	}
	if (WaitForSymbolResolverCheckBox->IsChecked())
	{
		CmdLine.Append(TEXT(" -WaitForSymbolResolver"));
	}
	if (DisableFramerateThrottleCheckBox->IsChecked())
	{
		CmdLine.Append(TEXT(" -DisableFramerateThrottle"));
	};
	if (InsightsTestCheckBox->IsChecked())
	{
		CmdLine.Append(TEXT(" -InsightsTest"));
	}
	if (NoUICheckBox->IsChecked())
	{
		CmdLine.Append(TEXT(" -NoUI"));
	}
	if (DebugToolsCheckBox->IsChecked())
	{
		CmdLine.Append(TEXT(" -DebugTools"));
	}

	FText AdditionalParams = DirectTraceAdditionalParamsTextBox->GetText();
	if (!AdditionalParams.IsEmpty())
	{
		CmdLine.Append(TEXT(" "));
		CmdLine.Append(AdditionalParams.ToString());
	}

	UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Start analysis (in separate process) for a direct trace (listen on port %u)", Port);
	FMiscUtils::OpenUnrealInsights(*CmdLine);

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply SConnectionWindow::Connect_OnClicked()
{
	FText TraceRecorderAddressText = TraceRecorderAddressTextBox->GetText();
	if (TraceRecorderAddressText.IsEmptyOrWhitespace())
	{
		// nothing to do
		return FReply::Handled();
	}
	const FString& TraceRecorderAddressStr = TraceRecorderAddressText.ToString();

	FText RunningInstanceAddressText = RunningInstanceAddressTextBox->GetText();
	if (RunningInstanceAddressText.IsEmptyOrWhitespace())
	{
		// nothing to do
		return FReply::Handled();
	}
	const FString& RunningInstanceAddressStr = RunningInstanceAddressText.ToString();

	FGraphEventArray Prerequisites;
	FGraphEventArray* PrerequisitesPtr = nullptr;
	if (ConnectTask.IsValid())
	{
		Prerequisites.Add(ConnectTask);
		PrerequisitesPtr = &Prerequisites;
	}

	const FString ChannelsExpandedStr = ChannelsTextBox->GetText().ToString().Replace(TEXT("default"), GInsightsDefaultChannelPreset);

	FGraphEventRef PreConnectTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
		[this, TraceRecorderAddressStr, RunningInstanceAddressStr, ChannelsExpandedStr]
		{
			bIsConnecting = true;

			UE_LOGF(LogInsightsFrontend, Log, "[Connection] Try connecting to \"%ls\"...", *RunningInstanceAddressStr);

			UE::Trace::FControlClient ControlClient;
			FString IPStr, PortStr;
			RunningInstanceAddressStr.Split(TEXT(":"), &IPStr, &PortStr);
			uint16 Port = 1985;
			if (!PortStr.IsEmpty())
			{
				Port = uint16(FCString::Atoi(*PortStr));
			}
			else
			{
				IPStr = RunningInstanceAddressStr;
			}
			if (ControlClient.Connect(*IPStr, Port))
			{
				UE_LOGF(LogInsightsFrontend, Log, "[Connection] SendSendTo(\"%ls\")...", *TraceRecorderAddressStr);
				ControlClient.SendSendTo(*TraceRecorderAddressStr);
				UE_LOGF(LogInsightsFrontend, Log, "[Connection] ToggleChannel(\"%ls\")...", *ChannelsExpandedStr);
				ControlClient.SendToggleChannel(*ChannelsExpandedStr, true);
				bIsConnectedSuccessfully = true;
			}
			else
			{
				bIsConnectedSuccessfully = false;
			}
		},
		TStatId{}, PrerequisitesPtr, ENamedThreads::AnyBackgroundThreadNormalTask);

	ConnectTask = FFunctionGraphTask::CreateAndDispatchWhenReady(
		[this, RunningInstanceAddressStr]
		{
			if (bIsConnectedSuccessfully)
			{
				UE_LOGF(LogInsightsFrontend, Log, "[Connection] Successfully connected.");

				FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ConnectSuccess", "Successfully connected to \"{0}\"!"), FText::FromString(RunningInstanceAddressStr)));
				NotificationInfo.bFireAndForget = false;
				NotificationInfo.bUseLargeFont = false;
				NotificationInfo.bUseSuccessFailIcons = true;
				NotificationInfo.ExpireDuration = 10.0f;
				TSharedRef<SNotificationItem> NotificationItem = NotificationList->AddNotification(NotificationInfo);
				NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
				NotificationItem->ExpireAndFadeout();
			}
			else
			{
				UE_LOGF(LogInsightsFrontend, Warning, "[Connection] Failed to connect to \"%ls\"!", *RunningInstanceAddressStr);

				FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("ConnectFailed", "Failed to connect to \"{0}\"!"), FText::FromString(RunningInstanceAddressStr)));
				NotificationInfo.bFireAndForget = false;
				NotificationInfo.bUseLargeFont = false;
				NotificationInfo.bUseSuccessFailIcons = true;
				NotificationInfo.ExpireDuration = 10.0f;
				TSharedRef<SNotificationItem> NotificationItem = NotificationList->AddNotification(NotificationInfo);
				NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
				NotificationItem->ExpireAndFadeout();
			}

			bIsConnecting = false;
		},
		TStatId{}, PreConnectTask, ENamedThreads::GameThread);

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
