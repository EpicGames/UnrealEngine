// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Async/TaskGraphFwd.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

#include <atomic>

class SVerticalBox;
class SCheckBox;
class SEditableTextBox;
class SNotificationList;

namespace UE::Trace
{
	class FStoreConnection;
}

namespace UE::Insights
{

/** Implements the Connection window. */
class SConnectionWindow : public SCompoundWidget
{
public:
	SConnectionWindow();
	virtual ~SConnectionWindow();

	SLATE_BEGIN_ARGS(SConnectionWindow) {}
	SLATE_END_ARGS()

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, TSharedRef<UE::Trace::FStoreConnection> InTraceStoreConnection);

private:
	TSharedRef<SWidget> ConstructConnectPanel();
	FReply StartUnrealInsightsDirectTrace_OnClicked();
	FReply Connect_OnClicked();

private:
	TSharedPtr<UE::Trace::FStoreConnection> TraceStoreConnection;

	TSharedPtr<SEditableTextBox> DirectTracePortTextBox;
	TSharedPtr<SEditableTextBox> DirectTraceAdditionalParamsTextBox;
	TSharedPtr<SEditableTextBox> TraceRecorderAddressTextBox;
	TSharedPtr<SEditableTextBox> RunningInstanceAddressTextBox;
	TSharedPtr<SEditableTextBox> ChannelsTextBox;

	TSharedPtr<SCheckBox> AutoQuitCheckBox;
	TSharedPtr<SCheckBox> WaitForSymbolResolverCheckBox;
	TSharedPtr<SCheckBox> DisableFramerateThrottleCheckBox;
	TSharedPtr<SCheckBox> InsightsTestCheckBox;
	TSharedPtr<SCheckBox> NoUICheckBox;
	TSharedPtr<SCheckBox> DebugToolsCheckBox;

	/** Widget for the non-intrusive notifications. */
	TSharedPtr<SNotificationList> NotificationList;

	FGraphEventRef ConnectTask;

	std::atomic<bool> bIsConnecting = false;
	std::atomic<bool> bIsConnectedSuccessfully = false;
};

} // namespace UE::Insights
