// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceDirectoryItem.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Logging/MessageLog.h"
#include "Misc/PathViews.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

// TraceAnalysis
#include "Trace/StoreClient.h"
#include "Trace/StoreConnection.h"

// TraceInsightsCore
#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Common/MessageDialogUtils.h"

// TraceInsightsFrontend
#include "InsightsFrontend/Common/InsightsFrontendStyle.h"
#include "InsightsFrontend/Common/Log.h"
#include "InsightsFrontend/Widgets/STraceStoreWindow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::STraceDirectoryItem"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STraceDirectoryItem::Construct(const FArguments& InArgs, TSharedPtr<FTraceDirectoryModel> InModel, STraceStoreWindow* InWindow)
{
	Model = InModel;
	Window = InWindow;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(2.0f, 1.0f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SBox)
				.HeightOverride(16.0f)
				.WidthOverride(16.0f)
				[
					SNew(SImage)
					.Image(FInsightsFrontendStyle::Get().GetBrush("Icons.UTraceStack"))
					.ColorAndOpacity(GetColor())
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(4.0f, 2.0f)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Model->Path))
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 2.0f)
			.AutoWidth()
			[
				ConstructOperations()
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STraceDirectoryItem::GetColor() const
{
	if (Model && Model->Color != NAME_None)
	{
		return FAppStyle::Get().GetSlateColor(Model->Color);
	}
	return FSlateColor::UseForeground();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STraceDirectoryItem::ConstructOperations()
{
	TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

	if (Model && EnumHasAllFlags(Model->Operations, ETraceDirOperations::ModifyStore))
	{
		Box->AddSlot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText_Raw(this, &STraceDirectoryItem::ModifyStoreTooltip)
				.OnClicked_Raw(this, &STraceDirectoryItem::OnModifyStore)
				.IsEnabled_Raw(this, &STraceDirectoryItem::CanModifyStore)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Edit"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}
	if (Model && EnumHasAllFlags(Model->Operations, ETraceDirOperations::Delete))
	{
		Box->AddSlot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("WatchDirsRemoveTooltip", "Removes the monitored directory. Files will not be deleted."))
				.OnClicked_Raw(this, &STraceDirectoryItem::OnDelete)
				.IsEnabled_Raw(this, &STraceDirectoryItem::CanDelete)
				[
					SNew(SImage)
					.Image(FInsightsFrontendStyle::Get().GetBrush("Icons.RemoveWatchDir"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}
	if (Model && EnumHasAllFlags(Model->Operations, ETraceDirOperations::Explore))
	{
		// If it has a Delete button then it is a "monitored directory".
		bool bIsWatchDir = (Model && EnumHasAllFlags(Model->Operations, ETraceDirOperations::Delete));

		Box->AddSlot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(bIsWatchDir ?
								LOCTEXT("ExploreWatchDirButtonToolTip", "Explores the monitored directory.") :
								LOCTEXT("ExploreTraceStoreDirButtonToolTip", "Explores the Trace Store Directory."))
				.OnClicked_Raw(this, &STraceDirectoryItem::OnExplore)
				.IsEnabled_Raw(this, &STraceDirectoryItem::CanExplore)
				[
					SNew(SImage)
					.Image(FInsightsCoreStyle::Get().GetBrush("Icons.FolderExplore"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return Box;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceDirectoryItem::CanExplore() const
{
	return Window && Window->GetTraceStoreConnection().CanChangeStoreSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceDirectoryItem::OnExplore()
{
	FSlateApplication::Get().CloseToolTip();
	if (Model)
	{
		FString FullPath(FPaths::ConvertRelativePathToFull(Model->Path));
		FPlatformProcess::ExploreFolder(*FullPath);
	}
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceDirectoryItem::CanDelete() const
{
	return Window && Window->GetTraceStoreConnection().CanChangeStoreSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceDirectoryItem::OnDelete()
{
	FSlateApplication::Get().CloseToolTip();

	// Avoid executing more than one operation
	if (bInOperation)
	{
		return FReply::Handled();
	}

	TGuardValue<bool> OperationGuard(bInOperation, true);

	if (Model)
	{
		EDialogResponse Response = FMessageDialogUtils::ShowChoiceDialog(LOCTEXT("MonitoredDirRemoveConfirmTitle", "Confirm removing monitored directory"), FText::Format(
			LOCTEXT("MonitoredDirRemoveConfirmFmt", "This will remove \"{0}\" from monitored directories.\n\nConfirm removing monitored directory?"),
			FText::FromString(Model->Path)));

		if (Response == EDialogResponse::OK)
		{
			UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Removing monitored directory: \"%ls\"...", *Model->Path);

			UE::Trace::FStoreClient* StoreClient = Window->GetTraceStoreConnection().GetStoreClient();
			if (!StoreClient ||
				!StoreClient->SetStoreDirectories(nullptr, {}, { *Model->Path }))
			{
				FName LogListingName(TEXT("UnrealInsights"));
				FMessageLog(LogListingName).Error(LOCTEXT("StoreCommunicationFail", "Failed to change settings on the store service."));
			}
		}
	}

	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceDirectoryItem::CanModifyStore() const
{
	return Window &&
		!Window->HasAnyLiveTrace() &&
		Window->GetTraceStoreConnection().CanChangeStoreSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceDirectoryItem::OnModifyStore()
{
	FSlateApplication::Get().CloseToolTip();

	// Avoid executing more than one operation
	if (bInOperation)
	{
		return FReply::Handled();
	}
	TGuardValue<bool> OperationGuard(bInOperation, true);

	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const FString Title = LOCTEXT("SetTraceStoreDirectory_DialogTitle", "Set Trace Store Directory").ToString();

		FString CurrentStoreDirectory = Window->GetStoreDirectory();
		FString SelectedDirectory;
		const bool bHasSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			Title,
			CurrentStoreDirectory,
			SelectedDirectory);

		const bool bIsWatchDir = Window->WatchDirectoriesModel.FindByPredicate([&](const auto& Directory) { return FPathViews::Equals(SelectedDirectory, Directory->Path); }) != nullptr;
		const bool bIsCurrentStoreDir = FPathViews::Equals(SelectedDirectory, CurrentStoreDirectory);

		if (bHasSelected && !bIsCurrentStoreDir)
		{
			FPaths::MakePlatformFilename(SelectedDirectory);
			FPaths::MakePlatformFilename(CurrentStoreDirectory);
			const TArray<FString> AddWatchDirs = { CurrentStoreDirectory };
			TArray<FString> RemoveWatchDirs;
			if (bIsWatchDir)
			{
				// If we are selecting a monitored dir as new store dir, make
				// sure we remove it as monitored directory.
				RemoveWatchDirs.Emplace(SelectedDirectory);
			}

			UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Changing store directory: \"%ls\"...", *SelectedDirectory);

			UE::Trace::FStoreClient* StoreClient = Window->GetTraceStoreConnection().GetStoreClient();
			if (!StoreClient ||
				!StoreClient->SetStoreDirectories(*SelectedDirectory, AddWatchDirs, RemoveWatchDirs))
			{
				FName LogListingName(TEXT("UnrealInsights"));
				FMessageLog(LogListingName).Error(LOCTEXT("StoreCommunicationFail", "Failed to change settings on the store service."));
			}
		}
	}
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceDirectoryItem::ModifyStoreTooltip() const
{
	return CanModifyStore() ?
		LOCTEXT("SetTraceStoreDirButtonToolTip", "Sets the Trace Store Directory.") :
		LOCTEXT("SetTraceStoreDirButtonTooltipInactive", "Sets the Trace Store Directory.\nNot available while live trace sessions are running.");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
