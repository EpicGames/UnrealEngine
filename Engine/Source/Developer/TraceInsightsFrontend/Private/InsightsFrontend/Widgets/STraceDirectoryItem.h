// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Insights
{

class STraceStoreWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

enum class ETraceDirOperations : uint8
{
	None = 0,
	ModifyStore = 1 << 0,
	Delete = 1 << 1,
	Explore = 1 << 2,
};
ENUM_CLASS_FLAGS(ETraceDirOperations);

////////////////////////////////////////////////////////////////////////////////////////////////////

/** View model for a source of traces. This could be the trace store default directory or an additional monitored directory. */
struct FTraceDirectoryModel
{
	FTraceDirectoryModel(FString&& InPath, const FName& InColor, ETraceDirOperations InOperations)
		: Path(MoveTemp(InPath)), Color(InColor), Operations(InOperations) {}

	/** Path to directory */
	FString Path;
	/** Assigned color */
	FName Color;
	/** Supported operations */
	ETraceDirOperations Operations;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class STraceDirectoryItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STraceDirectoryItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FTraceDirectoryModel> InClientItem, STraceStoreWindow* InWindow);

private:
	TSharedRef<SWidget> ConstructOperations();

	FSlateColor GetColor() const;

	bool CanModifyStore() const;
	FReply OnModifyStore();
	FText ModifyStoreTooltip() const;

	bool CanExplore() const;
	FReply OnExplore();

	bool CanDelete() const;
	FReply OnDelete();

private:
	bool bInOperation = false;
	STraceStoreWindow* Window = nullptr;
	TSharedPtr<FTraceDirectoryModel> Model;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
