// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceListRow.h"

#include "Framework/MetaData/DriverMetaData.h"
#include "HAL/FileManager.h"
#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SGridPanel.h"

// TraceInsightsCore
#include "InsightsCore/Common/InsightsCoreStyle.h"

// TraceInsightsFrontend
#include "InsightsFrontend/Common/InsightsFrontendStyle.h"
#include "InsightsFrontend/Common/Log.h"
#include "InsightsFrontend/ViewModels/TraceViewModel.h"
#include "InsightsFrontend/Widgets/STraceStoreWindow.h"

#define LOCTEXT_NAMESPACE "UE::Insights::STraceListRow"

namespace UE::Insights
{

////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FTraceListColumns::Date(TEXT("Date"));
const FName FTraceListColumns::Name(TEXT("Name"));
const FName FTraceListColumns::Uri(TEXT("Uri"));
const FName FTraceListColumns::Platform(TEXT("Platform"));
const FName FTraceListColumns::AppName(TEXT("AppName"));
const FName FTraceListColumns::BuildConfig(TEXT("BuildConfig"));
const FName FTraceListColumns::BuildTarget(TEXT("BuildTarget"));
const FName FTraceListColumns::BuildVersion(TEXT("BuildVersion"));
const FName FTraceListColumns::BuildBranch(TEXT("BuildBranch"));
const FName FTraceListColumns::Changelist(TEXT("Changelist"));
const FName FTraceListColumns::EngineVersion(TEXT("EngineVersion"));
const FName FTraceListColumns::Size(TEXT("Size"));
const FName FTraceListColumns::Status(TEXT("Status"));

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceListRow::Construct(const FArguments& InArgs, TSharedPtr<FTraceViewModel> InTrace, TSharedRef<STraceStoreWindow> InParentWidget, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	WeakTrace = InTrace.ToWeakPtr();
	WeakParentWidget = InParentWidget.ToWeakPtr();

	SMultiColumnTableRow<TSharedPtr<FTraceViewModel>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STraceListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == FTraceListColumns::Name)
	{
		TSharedPtr<SEditableTextBox> RenameTextBox;

		TSharedRef<SWidget> Widget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			.Padding(FMargin(2.0f))
			[
				SNew(SImage)
				.Image(FInsightsFrontendStyle::Get().GetBrush("Icons.UTrace"))
				.ColorAndOpacity(this, &STraceListRow::GetColorForPath)
			]
		]

		+ SHorizontalBox::Slot()
		.Padding(4.0f, 0.0f)
		.AutoWidth()
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(STextBlock)
				.Visibility_Lambda([this]() { return IsRenaming() ? EVisibility::Collapsed : EVisibility::Visible; })
				.Text(this, &STraceListRow::GetTraceName)
				.ColorAndOpacity(this, &STraceListRow::GetTraceTextColor)
				.HighlightText(this, &STraceListRow::GetTraceNameHighlightText)
				.HighlightColor(FLinearColor(0.75f, 0.75f, 0.75f, 1.0f))
				//.HighlightShape(FInsightsCoreStyle::Get().GetBrush("DarkGreenBrush"))
				.ToolTip(STraceListRow::GetTraceTooltip())
				.AddMetaData(FDriverMetaData::Id("TraceList"))
			]

			+ SOverlay::Slot()
			[
				SAssignNew(RenameTextBox, SEditableTextBox)
				.Padding(0.0f)
				.Visibility_Lambda([this]() { return IsRenaming() ? EVisibility::Visible : EVisibility::Collapsed; })
				.Text(this, &STraceListRow::GetTraceName)
				.RevertTextOnEscape(true)
				.ClearKeyboardFocusOnCommit(true)
				.OnTextCommitted(this, &STraceListRow::RenameTextBox_OnValueCommitted)
				.ToolTip(STraceListRow::GetTraceTooltip())
			]
		];

		TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
		if (TracePin.IsValid())
		{
			// A weak reference is stored in the view model in order to focus the text box
			// when starts renaming.
			TracePin->SetRenameTextBox(RenameTextBox.ToWeakPtr());
		}

		return Widget;
	}
	else if (ColumnName == FTraceListColumns::Uri)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceUri)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::Platform)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTracePlatform)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::AppName)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceAppName)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::BuildConfig)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceBuildConfiguration)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::BuildTarget)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceBuildTarget)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::BuildVersion)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceBuildVersion)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::BuildBranch)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceBranch)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::Changelist)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceChangelist)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::EngineVersion)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceEngineVersion)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::Size)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceSize)
				.ColorAndOpacity(this, &STraceListRow::GetColorBySize)
				.ToolTip(STraceListRow::GetTraceTooltip())
			];
	}
	else if (ColumnName == FTraceListColumns::Status)
	{
		return SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceListRow::GetTraceStatus)
				.ToolTip(STraceListRow::GetTraceTooltip())
				.AddMetaData(FDriverMetaData::Id("TraceStatusColumnList"))
				.ColorAndOpacity(FStyleColors::AccentRed)
			];
	}
	else
	{
		return SNew(STextBlock).Text(LOCTEXT("UnknownColumn", "Unknown Column"));
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceListRow::IsRenaming() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->IsRenaming();
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceListRow::RenameTextBox_OnValueCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (InCommitType != ETextCommit::OnCleared)
		{
			Rename(*TracePin, InText);
		}
		TracePin->StopRenaming();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceListRow::Rename(FTraceViewModel& Trace, const FText& InText)
{
	TSharedPtr<STraceStoreWindow> TraceStoreWindow = WeakParentWidget.Pin();
	if (!TraceStoreWindow.IsValid())
	{
		return;
	}

	const FString TraceName = Trace.GetName();
	const FString NewTraceName = InText.ToString().TrimStartAndEnd();

	if (NewTraceName == TraceName || NewTraceName.IsEmpty())
	{
		return;
	}

	if (Trace.IsLive())
	{
		FText Message = LOCTEXT("RenameLive", "Cannot rename a live session!");
		TraceStoreWindow->ShowFailMessage(Message);
		return;
	}

	UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Renaming \"%ls\" to \"%ls\"...", *TraceName, *NewTraceName);

	const FString TraceFile = Trace.GetUri();
	const FString NewTraceFile = FPaths::Combine(FPaths::GetPath(TraceFile), NewTraceName + TEXT(".utrace"));

	FText Reason;
	bool bHasReservedCharacters = false;
	for (const TCHAR* P = *NewTraceName; *P; ++P)
	{
		if (*P == TEXT('/') || *P == TEXT('\\'))
		{
			bHasReservedCharacters = true;
			Reason = LOCTEXT("RenameReservedCharacters", "Name may not contain / or \\ characters.");
			break;
		}
	}
	if (bHasReservedCharacters || !FPaths::ValidatePath(NewTraceFile, &Reason))
	{
		FText Message = FText::Format(LOCTEXT("RenameFailFmt3", "Failed to rename \"{0}\" to \"{1}\"!\n{2}"), Trace.GetNameText(), FText::FromString(NewTraceName), Reason);
		TraceStoreWindow->ShowFailMessage(Message);
		return;
	}

	if (FPaths::FileExists(NewTraceFile))
	{
		UE_LOGF(LogInsightsFrontend, Warning, "[TraceStore] Failed to rename \"%ls\" to \"%ls\"! File already exists.", *TraceName, *NewTraceName);

		FText Message = FText::Format(LOCTEXT("RenameFailFmt1", "Failed to rename \"{0}\" to \"{1}\"!\nFile already exists."), Trace.GetNameText(), FText::FromString(NewTraceName));
		TraceStoreWindow->ShowFailMessage(Message);
		return;
	}

	if (!FPaths::FileExists(TraceFile) ||
		!IFileManager::Get().Move(*NewTraceFile, *TraceFile, false))
	{
		UE_LOGF(LogInsightsFrontend, Warning, "[TraceStore] Failed to rename \"%ls\" to \"%ls\"!", *TraceName, *NewTraceName);

		FText Message = FText::Format(LOCTEXT("RenameFailFmt2", "Failed to rename \"{0}\" to \"{1}\"!"), Trace.GetNameText(), FText::FromString(NewTraceName));
		TraceStoreWindow->ShowFailMessage(Message);
		return;
	}

	UE_LOGF(LogInsightsFrontend, Verbose, "[TraceStore] Renamed utrace file (\"%ls\").", *NewTraceFile);
	Trace.SetName(NewTraceName);
	Trace.SetUri(NewTraceFile);

	TraceStoreWindow->TraceViewModelMap.Remove(Trace.GetId());
	Trace.SetId(FTraceViewModel::InvalidTraceId); // cannot be open until its TraceId is updated
	Trace.SetChangeSerial(0); // to force update

	const FString CacheFile = FPaths::ChangeExtension(TraceFile, TEXT("ucache"));
	if (FPaths::FileExists(CacheFile))
	{
		const FString NewCacheFile = FPaths::Combine(FPaths::GetPath(CacheFile), NewTraceName + TEXT(".ucache"));
		if (IFileManager::Get().Move(*NewCacheFile, *CacheFile, true))
		{
			UE_LOGF(LogInsightsFrontend, Verbose, "[TraceStore] Renamed ucache file (\"%ls\").", *NewCacheFile);
		}
	}

	// Also rename the metadata ini file (<trace>.utrace.ini) if present.
	// Use Replace=false so a pre-existing sidecar at the destination (from an aborted
	// earlier rename or stale orphan) is not silently overwritten — warn instead.
	const FString SidecarFile = TraceFile + TEXT(".ini");
	if (FPaths::FileExists(SidecarFile))
	{
		const FString NewSidecarFile = NewTraceFile + TEXT(".ini");
		if (FPaths::FileExists(NewSidecarFile))
		{
			UE_LOGF(LogInsightsFrontend, Warning,
				"[TraceStore] Sidecar ini file already exists at destination, keeping both: \"%ls\" (old) and \"%ls\" (existing).",
				*SidecarFile, *NewSidecarFile);
		}
		else if (IFileManager::Get().Move(*NewSidecarFile, *SidecarFile, /*Replace*/ false))
		{
			UE_LOGF(LogInsightsFrontend, Verbose, "[TraceStore] Renamed sidecar ini file (\"%ls\").", *NewSidecarFile);
		}
	}

	FText Message = FText::Format(LOCTEXT("RenameSuccessFmt", "Renamed \"{0}\" to \"{1}\"."), FText::FromString(TraceName), FText::FromString(NewTraceName));
	TraceStoreWindow->ShowSuccessMessage(Message);

	TraceStoreWindow->bSetKeyboardFocusOnNextTick = true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceIndexAndId() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		const FString TraceIdStr = FString::Printf(TEXT("0x%X"), TracePin->GetId());
		return FText::FromString(TraceIdStr);
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceName() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetNameText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STraceListRow::GetTraceTextColor() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (!TracePin->HasValidId())
		{
			return FSlateColor(EStyleColor::White25);
		}
	}
	return IsSelected() || IsHovered() ?
		FSlateColor(EStyleColor::ForegroundHover) :
		FSlateColor(EStyleColor::Foreground);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceNameHighlightText() const
{
	TSharedPtr<STraceStoreWindow> ParentWidgetPin = WeakParentWidget.Pin();
	if (ParentWidgetPin.IsValid())
	{
		if (!ParentWidgetPin->bSearchByCommandLine && ParentWidgetPin->FilterByNameSearchBox.IsValid())
		{
			const FText SearchText = ParentWidgetPin->FilterByNameSearchBox->GetText();
			bool bQuotesRemoved = false;
			const FString SearchString = SearchText.ToString().TrimQuotes(&bQuotesRemoved);
			if (bQuotesRemoved)
			{
				return FText::FromString(SearchString);
			}
			else
			{
				return SearchText;
			}
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceUri() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetUriText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STraceListRow::GetColorForPath() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetDirectoryColor();
	}
	else
	{
		return FSlateColor::UseForeground();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTracePlatform() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetPlatformText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceAppName() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetAppNameText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceBuildConfiguration() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (TracePin->GetBuildConfiguration() != EBuildConfiguration::Unknown)
		{
			return EBuildConfigurations::ToText(TracePin->GetBuildConfiguration());
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceBuildTarget() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (TracePin->GetBuildTarget() != EBuildTargetType::Unknown)
		{
			return FText::FromString(LexToString(TracePin->GetBuildTarget()));
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceBuildVersion() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetBuildVersionText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceBranch() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetBuildBranchText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceChangelist() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return FText::AsNumber(TracePin->GetChangelist(), &FNumberFormattingOptions::DefaultNoGrouping());
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility STraceListRow::TraceChangelistVisibility() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetChangelist() != 0 ? EVisibility::Visible : EVisibility::Collapsed;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceEngineVersion() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetEngineVersionText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceCommandLine() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetCommandLineText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceCommandLineHighlightText() const
{
	TSharedPtr<STraceStoreWindow> ParentWidgetPin = WeakParentWidget.Pin();
	if (ParentWidgetPin.IsValid())
	{
		if (ParentWidgetPin->bSearchByCommandLine && ParentWidgetPin->FilterByNameSearchBox.IsValid())
		{
			const FText SearchText = ParentWidgetPin->FilterByNameSearchBox->GetText();
			bool bQuotesRemoved = false;
			const FString SearchString = SearchText.ToString().TrimQuotes(&bQuotesRemoved);
			if (bQuotesRemoved)
			{
				return FText::FromString(SearchString);
			}
			else
			{
				return SearchText;
			}
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceVFSPaths() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return TracePin->GetVFSPathsText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceTimestamp() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return FText::AsDate(TracePin->GetTimestamp());
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceTimestampForTooltip() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		return FText::AsDateTime(TracePin->GetTimestamp());
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceSize() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		//FNumberFormattingOptions FormattingOptions;
		//FormattingOptions.MinimumFractionalDigits = 1;
		//FormattingOptions.MaximumFractionalDigits = 1;
		//return FText::AsMemory(TracePin->Size, &FormattingOptions);
		return FText::Format(LOCTEXT("SessionFileSizeFormatKiB", "{0} KiB"), TracePin->GetSize() / 1024);
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceSizeForTooltip() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (TracePin->GetSize() > 1024)
		{
			return FText::Format(LOCTEXT("TraceTooltip_FileSize2", "{0} bytes ({1})"), FText::AsNumber(TracePin->GetSize()), FText::AsMemory(TracePin->GetSize()));
		}
		else
		{
			return FText::Format(LOCTEXT("TraceTooltip_FileSize1", "{0} bytes"), FText::AsNumber(TracePin->GetSize()));
		}
	}
	else
	{
		return FText::GetEmpty();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STraceListRow::GetColorBySize() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (TracePin->GetSize() < 1024ULL * 1024ULL)
		{
			// < 1 MiB
			TSharedRef<ITypedTableView<TSharedPtr<FTraceViewModel>>> OwnerWidget = OwnerTablePtr.Pin().ToSharedRef();
			const TSharedPtr<FTraceViewModel>* MyItem = OwnerWidget->Private_ItemFromWidget(this);
			const bool IsSelected = OwnerWidget->Private_IsItemSelected(*MyItem);
			if (IsSelected)
			{
				return FSlateColor(FLinearColor(0.75f, 0.75f, 0.75f, 1.0f));
			}
			else
			{
				return FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f));
			}
		}
		else if (TracePin->GetSize() < 1024ULL * 1024ULL * 1024ULL)
		{
			// [1 MiB  .. 1 GiB)
			return FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
		}
		else
		{
			// > 1 GiB
			return FSlateColor(FLinearColor(1.0f, 0.5f, 0.5f, 1.0f));
		}
	}
	else
	{
		return FSlateColor(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceStatus() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (TracePin->IsLive())
		{
			return LOCTEXT("LiveTraceStatus", "LIVE");
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceListRow::GetTraceStatusForTooltip() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		if (TracePin->IsLive())
		{
			const uint32 IpAddress = TracePin->GetIpAddress();
			FString Ip = FString::Printf(TEXT("%d.%d.%d.%d"),
				(IpAddress >> 24) & 0xFF,
				(IpAddress >> 16) & 0xFF,
				(IpAddress >> 8) & 0xFF,
				IpAddress & 0xFF);
			return FText::Format(LOCTEXT("LiveTraceStatusFmt", "LIVE ({0})"), FText::FromString(Ip));
		}
		else
		{
			return LOCTEXT("OfflineTraceStatus", "Offline");
		}
	}
	return FText::GetEmpty();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<IToolTip> STraceListRow::GetTraceTooltip() const
{
	return SNew(SLazyToolTip, SharedThis(this));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SToolTip> STraceListRow::CreateTooltip() const
{
	TSharedPtr<FTraceViewModel> TracePin = WeakTrace.Pin();
	if (TracePin.IsValid())
	{
		TSharedPtr<SGridPanel> GridPanel;
		TSharedPtr<SToolTip> TraceTooltip =
			SNew(SToolTip)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(FMargin(-7.0f, -7.0f, -7.0f, 0.0f))
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(FMargin(6.0f, 6.0f, 6.0f, 6.0f))
					.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FSlateColor(EStyleColor::Panel))
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.Padding(FMargin(2.0f, 2.0f, 2.0f, 2.0f))
						.FillWidth(1.0f)
						[
							SNew(STextBlock)
							.Text(this, &STraceListRow::GetTraceName)
							//.Font(FAppStyle::Get().GetFontStyle("Font.Large")) // 14
							.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
							.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
						]

						+ SHorizontalBox::Slot()
						.Padding(FMargin(2.0f, 2.0f, 2.0f, 2.0f))
						.AutoWidth()
						[
							SNew(STextBlock)
							//.Font(FAppStyle::Get().GetFontStyle("Font.Large")) // 14
							.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
							.Text(this, &STraceListRow::GetTraceIndexAndId)
							.ColorAndOpacity(FSlateColor(EStyleColor::White25))
						]
					]
				]

				+ SVerticalBox::Slot()
				.Padding(FMargin(-7.0f, 1.0f, -7.0f, 0.0f))
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(FMargin(6.0f, 6.0f, 6.0f, 4.0f))
					.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FSlateColor(EStyleColor::Panel))
					[
						SNew(STextBlock)
						.Text(this, &STraceListRow::GetTraceUri)
						//.Font(FAppStyle::Get().GetFontStyle("SmallFont")) // 8
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 9))
						.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(-7.0f, 0.0f, -7.0f, -7.0f))
				[
					SNew(SBorder)
					.Padding(FMargin(6.0f, 0.0f, 6.0f, 4.0f))
					.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FSlateColor(EStyleColor::Panel))
					[
						SAssignNew(GridPanel, SGridPanel)
					]
				]
			];

		int32 Row = 0;
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_CommandLine", "Command Line:"), &STraceListRow::GetTraceCommandLine, &STraceListRow::GetTraceCommandLineHighlightText);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_Platform", "Platform:"), &STraceListRow::GetTracePlatform);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_AppName", "App Name:"), &STraceListRow::GetTraceAppName);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_BuildConfig", "Build Config:"), &STraceListRow::GetTraceBuildConfiguration);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_BuildTarget", "Build Target:"), &STraceListRow::GetTraceBuildTarget);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_BuildVersion", "Build Version:"), &STraceListRow::GetTraceBuildVersion);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_Branch", "Build Branch:"), &STraceListRow::GetTraceBranch);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_Changelist", "Changelist:"), &STraceListRow::GetTraceChangelist, nullptr, &STraceListRow::TraceChangelistVisibility);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_EngineVersion", "Engine Version:"), &STraceListRow::GetTraceEngineVersion);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_VFSPaths", "VFS Paths:"), &STraceListRow::GetTraceVFSPaths);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_Timestamp", "Timestamp:"), &STraceListRow::GetTraceTimestampForTooltip);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_Size", "File Size:"), &STraceListRow::GetTraceSizeForTooltip);
		AddGridPanelRow(GridPanel, Row++, LOCTEXT("TraceTooltip_Status", "Status:"), &STraceListRow::GetTraceStatusForTooltip);

		return TraceTooltip;
	}
	else
	{
		TSharedPtr<SToolTip> TraceTooltip =
			SNew(SToolTip)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TraceTooltip_NA", "N/A"))
			];

		return TraceTooltip;
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STraceListRow::AddGridPanelRow(TSharedPtr<SGridPanel> Grid, int32 Row, const FText& InHeaderText,
	typename TAttribute<FText>::FGetter::template TConstMethodPtr<STraceListRow> InValueTextFn,
	typename TAttribute<FText>::FGetter::template TConstMethodPtr<STraceListRow> InHighlightTextFn,
	typename TAttribute<EVisibility>::FGetter::template TConstMethodPtr<STraceListRow> InVisibilityFn) const
{
	SGridPanel::FSlot* Slot0 = nullptr;
	Grid->AddSlot(0, Row)
		.Expose(Slot0)
		.Padding(2.0f)
		.HAlign(HAlign_Right)
		[
			SNew(STextBlock)
			.Text(InHeaderText)
			.ColorAndOpacity(FSlateColor(EStyleColor::White25))
		];

	SGridPanel::FSlot* Slot1 = nullptr;

	if (InHighlightTextFn)
	{
		Grid->AddSlot(1, Row)
			.Expose(Slot1)
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(this, InValueTextFn)
				.HighlightText(this, InHighlightTextFn)
				.HighlightShape(FInsightsCoreStyle::Get().GetBrush("DarkGreenBrush"))
				.WrapTextAt(1024.0f)
				.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
				.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
			];
	}
	else
	{
		Grid->AddSlot(1, Row)
			.Expose(Slot1)
			.Padding(2.0f)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(this, InValueTextFn)
				.WrapTextAt(1024.0f)
				.WrappingPolicy(ETextWrappingPolicy::AllowPerCharacterWrapping)
				.ColorAndOpacity(FSlateColor(EStyleColor::Foreground))
			];
	}

	if (InVisibilityFn)
	{
		Slot0->GetWidget()->SetVisibility(MakeAttributeSP(this, InVisibilityFn));
		Slot1->GetWidget()->SetVisibility(MakeAttributeSP(this, InVisibilityFn));
	}
	else
	{
		auto Fn = MakeAttributeSP(this, InValueTextFn);
		Slot0->GetWidget()->SetVisibility(MakeAttributeLambda([this, Fn]() { return Fn.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }));
		Slot1->GetWidget()->SetVisibility(MakeAttributeLambda([this, Fn]() { return Fn.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; }));
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE