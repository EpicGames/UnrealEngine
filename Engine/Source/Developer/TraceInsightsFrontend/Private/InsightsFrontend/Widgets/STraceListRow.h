// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

// TraceInsightsCore
#include "InsightsCore/Widgets/SLazyToolTip.h"

class SGridPanel;

namespace UE::Insights
{

class FTraceViewModel;
class STraceStoreWindow;

////////////////////////////////////////////////////////////////////////////////////////////////////

struct FTraceListColumns
{
	static const FName Date;
	static const FName Name;
	static const FName Uri;
	static const FName Platform;
	static const FName AppName;
	static const FName BuildConfig;
	static const FName BuildTarget;
	static const FName BuildVersion;
	static const FName BuildBranch;
	static const FName Changelist;
	static const FName EngineVersion;
	static const FName Size;
	static const FName Status;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class STraceListRow : public SMultiColumnTableRow<TSharedPtr<FTraceViewModel>>, public ILazyToolTipCreator
{
	SLATE_BEGIN_ARGS(STraceListRow) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FTraceViewModel> InTrace, TSharedRef<STraceStoreWindow> InParentWidget, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	bool IsRenaming() const;
	void RenameTextBox_OnValueCommitted(const FText& InText, ETextCommit::Type InCommitType);
	void Rename(FTraceViewModel& Trace, const FText& InText);

	FText GetTraceIndexAndId() const;
	FText GetTraceName() const;
	FSlateColor GetTraceTextColor() const;
	FText GetTraceNameHighlightText() const;
	FText GetTraceUri() const;
	FSlateColor GetColorForPath() const;
	FText GetTracePlatform() const;
	FText GetTraceAppName() const;
	FText GetTraceEngineVersion() const;
	FText GetTraceBuildVersion() const;
	FText GetTraceBranch() const;
	FText GetTraceChangelist() const;
	EVisibility TraceChangelistVisibility() const;
	FText GetTraceBuildConfiguration() const;
	FText GetTraceBuildTarget() const;
	FText GetTraceCommandLine() const;
	FText GetTraceCommandLineHighlightText() const;
	FText GetTraceVFSPaths() const;
	FText GetTraceTimestamp() const;
	FText GetTraceTimestampForTooltip() const;
	FText GetTraceSize() const;
	FText GetTraceSizeForTooltip() const;
	FSlateColor GetColorBySize() const;
	FText GetTraceStatus() const;
	FText GetTraceStatusForTooltip() const;
	TSharedPtr<IToolTip> GetTraceTooltip() const;

	// ILazyToolTipCreator
	virtual TSharedPtr<SToolTip> CreateTooltip() const override;

private:
	void AddGridPanelRow(TSharedPtr<SGridPanel> Grid, int32 Row, const FText& InHeaderText,
		typename TAttribute<FText>::FGetter::template TConstMethodPtr<STraceListRow> InValueTextFn,
		typename TAttribute<FText>::FGetter::template TConstMethodPtr<STraceListRow> InHighlightTextFn = nullptr,
		typename TAttribute<EVisibility>::FGetter::template TConstMethodPtr<STraceListRow> InVisibilityFn = nullptr) const;

private:
	TWeakPtr<FTraceViewModel> WeakTrace;
	TWeakPtr<STraceStoreWindow> WeakParentWidget;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
