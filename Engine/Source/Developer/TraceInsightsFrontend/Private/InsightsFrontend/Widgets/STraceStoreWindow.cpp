// Copyright Epic Games, Inc. All Rights Reserved.

#include "STraceStoreWindow.h"

#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MetaData/DriverMetaData.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Text.h"
#include "Logging/MessageLog.h"
#include "Misc/MessageDialog.h"
#include "Misc/PathViews.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Testing/SStarshipSuite.h" // for RestoreStarshipSuite()
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
	#include "AnalyticsEventAttribute.h"
	#include "EngineAnalytics.h"
	#include "Interfaces/IAnalyticsProvider.h"
#endif // WITH_EDITOR

// TraceAnalysis
#include "Trace/StoreClient.h"
#include "Trace/StoreConnection.h"

// TraceInsightsCore
#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Common/MiscUtils.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Table/ViewModels/TableImporter.h"

// TraceInsightsFrontend
#include "InsightsFrontend/Common/InsightsFrontendStyle.h"
#include "InsightsFrontend/Common/Log.h"
#include "InsightsFrontend/InsightsFrontendSettings.h"
#include "InsightsFrontend/ITraceInsightsFrontendModule.h"
#include "InsightsFrontend/StoreService/StoreBrowser.h"
#include "InsightsFrontend/TraceInsightsFrontendModule.h"
#include "InsightsFrontend/ViewModels/TraceSetFilter.h"
#include "InsightsFrontend/ViewModels/TraceViewModel.h"
#include "InsightsFrontend/Widgets/STraceDirectoryItem.h"
#include "InsightsFrontend/Widgets/STraceListRow.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <handleapi.h> // for CreateEvent
#include <synchapi.h> // for CloseHandle
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define LOCTEXT_NAMESPACE "UE::Insights::STraceStoreWindow"

namespace UE::Insights
{

FName STraceStoreWindow::LogListingName(TEXT("InsightsFrontend"));

////////////////////////////////////////////////////////////////////////////////////////////////////
// STraceStoreWindow
////////////////////////////////////////////////////////////////////////////////////////////////////

STraceStoreWindow::STraceStoreWindow()
	: TableImporter(MakeShared<FTableImporter>(LogListingName))
{
	SortColumn = FTraceListColumns::Date;
	SortMode = EColumnSortMode::Ascending;

	// Add controls for the local server
	ServerControls.Emplace(TEXT("127.0.0.1"), 0, FAppStyle::Get().GetStyleSetName());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

STraceStoreWindow::~STraceStoreWindow()
{
	if (OnTickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(OnTickHandle);
	}

#if WITH_EDITOR
	if (DurationActive > 0.0f && FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Insights.Usage.SessionBrowser"), FAnalyticsEventAttribute(TEXT("Duration"), DurationActive));
	}
#endif // WITH_EDITOR

	DisableAutoConnect();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STraceStoreWindow::Construct(const FArguments& InArgs, TSharedRef<UE::Trace::FStoreConnection> InTraceStoreConnection)
{
	TraceStoreConnection = InTraceStoreConnection;
	StoreBrowser.Reset(new FStoreBrowser(InTraceStoreConnection));

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SBox)
			[
				SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(0.0f)
				.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				.BorderBackgroundColor(FSlateColor(EStyleColor::Panel))
			]
		]

		// Overlay slot for the main window area
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(MainContentPanel, SVerticalBox)

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoHeight()
			.Padding(6.0f, 8.0f, 12.0f, 0.0f)
			[
				ConstructTraceStoreDirectoryPanel()
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			[
				ConstructFiltersToolbar()
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillHeight(1.0f)
			.Padding(3.0f, 0.0f, 3.0f, 4.0f)
			[
				ConstructSessionsPanel()
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.AutoHeight()
			.Padding(12.0f, 4.0f, 12.0f, 8.0f)
			[
				ConstructLoadPanel()
			]
		]

		// Overlay for fake splash-screen.
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(0.0f)
		[
			SNew(SBox)
			.Visibility(this, &STraceStoreWindow::SplashScreenOverlay_Visibility)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("PopupText.Background"))
				.BorderBackgroundColor(this, &STraceStoreWindow::SplashScreenOverlay_ColorAndOpacity)
				.Padding(0.0f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &STraceStoreWindow::GetSplashScreenOverlayText)
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						.ColorAndOpacity(this, &STraceStoreWindow::SplashScreenOverlay_TextColorAndOpacity)
					]
				]
			]
		]

		// Notification area overlay
		+ SOverlay::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(16.0f)
		[
			SAssignNew(NotificationList, SNotificationList)
		]

		// Settings dialog overlay
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Expose(OverlaySettingsSlot)
	];

	// Register tick functions.
	OnTick = FTickerDelegate::CreateSP(this, &STraceStoreWindow::CoreTick);
	OnTickHandle = FTSTicker::GetCoreTicker().AddTicker(OnTick, 0.0f);

	CreateFilters();

	if (StoreHostTextBox)
	{
		StoreHostTextBox->SetText(FText::FromString(TraceStoreConnection->GetLastStoreHost()));
	}

	RefreshTraceList();

	if (AutoConnect_IsChecked() == ECheckBoxState::Checked)
	{
		EnableAutoConnect();
	}

	bSetKeyboardFocusOnNextTick = true;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STraceStoreWindow::ConstructFiltersToolbar()
{
	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.SetStyle(&FInsightsCoreStyle::Get(), "SecondaryToolbar");

	ToolbarBuilder.BeginSection("Filters");
	{
		// Toggle between filtering the list of trace sessions by name or by command line
		ToolbarBuilder.AddWidget(
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.HAlign(HAlign_Center)
			.Padding(3.0f)
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) { bSearchByCommandLine = (NewState == ECheckBoxState::Checked); OnFilterChanged(); })
			.IsChecked_Lambda([this]() { return bSearchByCommandLine ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.ToolTipText(LOCTEXT("ToggleNameFilter_Tooltip", "Toggle between filtering the list of trace sessions by name or by command line."))
			[
				SNew(SBox)
				.Padding(1.0f)
				[
					SNew(SImage)
					.Image(FInsightsFrontendStyle::Get().GetBrush("Icons.Console"))
				]
			]);

		// Text Filter (Search Box)
		ToolbarBuilder.AddWidget(
			SNew(SBox)
			.MaxDesiredWidth(400.0f)
			[
				SAssignNew(FilterByNameSearchBox, SSearchBox)
				.MinDesiredWidth(150.0f)
				.HintText_Lambda([this]()
					{
						return bSearchByCommandLine ?
							LOCTEXT("CmdLineFilter_Hint", "Command Line") :
							LOCTEXT("NameFilter_Hint", "Name");
					})
				.ToolTipText_Lambda([this]()
					{
						return bSearchByCommandLine ?
							LOCTEXT("CmdLineFilter_Tooltip", "Type here to filter the list of trace sessions by command line.") :
							LOCTEXT("NameFilter_Tooltip", "Type here to filter the list of trace sessions by name.");
					})
				.IsEnabled_Lambda([this]() { return TraceViewModels.Num() > 0; })
				.OnTextChanged(this, &STraceStoreWindow::FilterByNameSearchBox_OnTextChanged)
				.DelayChangeNotificationsWhileTyping(true)
			]);

		// Filter by Platform
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakePlatformFilterMenu),
			LOCTEXT("FilterByPlatformText", "Platform"),
			LOCTEXT("FilterByPlatformToolTip", "Filters the list of trace sessions by platform."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

		// Filter by AppName
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeAppNameFilterMenu),
			LOCTEXT("FilterByAppNameText", "App Name"),
			LOCTEXT("FilterByAppNameToolTip", "Filters the list of trace sessions by application name."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

		// Filter by Build Config
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeBuildConfigFilterMenu),
			LOCTEXT("FilterByBuildConfigText", "Config"),
			LOCTEXT("FilterByBuildConfigToolTip", "Filters the list of trace sessions by build configuration."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

		// Filter by Build Target
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeBuildTargetFilterMenu),
			LOCTEXT("FilterByBuildTargetText", "Target"),
			LOCTEXT("FilterByBuildTargetToolTip", "Filters the list of trace sessions by build target."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

		// Filter by Build Version
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeBuildVersionFilterMenu),
			LOCTEXT("FilterByBuildVersionText", "Build Version"),
			LOCTEXT("FilterByBuildVersionToolTip", "Filters the list of trace sessions by Build Version."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

		// Filter by Branch
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeBranchFilterMenu),
			LOCTEXT("FilterByBranchText", "Branch"),
			LOCTEXT("FilterByBranchToolTip", "Filters the list of trace sessions by branch."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

#if 0
		// Filter by Changelist
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeChangelistFilterMenu),
			LOCTEXT("FilterByChangelistText", "Changelist"),
			LOCTEXT("FilterByChangelistToolTip", "Filters the list of trace sessions by changelist."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);
#endif

		// Filter by Engine Version
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeEngineVersionFilterMenu),
			LOCTEXT("FilterByEngineVersionText", "Engine Version"),
			LOCTEXT("FilterByEngineVersionToolTip", "Filters the list of trace sessions by Engine Version."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

		// Filter by Size
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeSizeFilterMenu),
			LOCTEXT("FilterBySizeText", "Size"),
			LOCTEXT("FilterBySizeToolTip", "Filters the list of trace sessions by size."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);

		// Filter by Status
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &STraceStoreWindow::MakeStatusFilterMenu),
			LOCTEXT("FilterByStatusText", "Status"),
			LOCTEXT("FilterByStatusToolTip", "Filters the list of trace sessions by status.."),
			FSlateIcon(FInsightsFrontendStyle::GetStyleSetName(), "Icons.Filter.ToolBar"),
			false);
	}
	ToolbarBuilder.EndSection();

	FSlimHorizontalToolBarBuilder RightSideToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	RightSideToolbarBuilder.SetStyle(&FInsightsCoreStyle::Get(), "PrimaryToolbar");
	RightSideToolbarBuilder.BeginSection("FilterStats");
	{
		// Filter Stats Text (number and size of filtered trace sessions)
		RightSideToolbarBuilder.AddWidget(
			SNew(SBox)
			.VAlign(VAlign_Bottom)
			.Padding(FMargin(4.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(STextBlock)
				.Text(this, &STraceStoreWindow::GetFilterStatsText)
			]);
	}
	RightSideToolbarBuilder.EndSection();

	return SNew(SHorizontalBox)
		.Visibility(this, &STraceStoreWindow::VisibleIfConnected)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.FillWidth(1.0f)
		.Padding(0.0f)
		[
			ToolbarBuilder.MakeWidget()
		]

	+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.0f)
		[
			RightSideToolbarBuilder.MakeWidget()
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STraceStoreWindow::ConstructSessionsPanel()
{
	TSharedRef<SWidget> Widget = SAssignNew(TraceListView, SListView<TSharedPtr<FTraceViewModel>>)
		.Visibility(this, &STraceStoreWindow::HiddenIfNotConnected)
		.IsFocusable(true)
		.SelectionMode(ESelectionMode::Multi)
		.OnSelectionChanged(this, &STraceStoreWindow::TraceList_OnSelectionChanged)
		.OnMouseButtonDoubleClick(this, &STraceStoreWindow::TraceList_OnMouseButtonDoubleClick)
		.ListItemsSource(&FilteredTraceViewModels)
		.OnGenerateRow(this, &STraceStoreWindow::TraceList_OnGenerateRow)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		.OnContextMenuOpening(FOnContextMenuOpening::CreateSP(this, &STraceStoreWindow::TraceList_GetMenuContent))
		.HeaderRow
		(
			SNew(SHeaderRow)

			+ SHeaderRow::Column(FTraceListColumns::Name)
			.FillWidth(0.25f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::Name)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(24.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NameColumn", "Name"))
					.ColorAndOpacity_Lambda([this] { return FilterByName->GetRawFilterText().IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::Platform)
			.FillWidth(0.1f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::Platform)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakePlatformColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(24.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PlatformColumn", "Platform"))
					.ColorAndOpacity_Lambda([this] { return FilterByPlatform->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::AppName)
			.FillWidth(0.1f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::AppName)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeAppNameColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(24.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AppNameColumn", "App Name"))
					.ColorAndOpacity_Lambda([this] { return FilterByAppName->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::BuildConfig)
			.FillWidth(0.1f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::BuildConfig)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeBuildConfigColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(24.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BuildConfigColumn", "Build Config"))
					.ColorAndOpacity_Lambda([this] { return FilterByBuildConfig->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::BuildTarget)
			.FillWidth(0.1f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::BuildTarget)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeBuildTargetColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(24.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BuildTargetColumn", "Build Target"))
					.ColorAndOpacity_Lambda([this] { return FilterByBuildTarget->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::BuildVersion)
			.FillWidth(0.25f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::BuildVersion)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeBuildVersionColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BuildVersionColumn", "Build Version"))
					.ColorAndOpacity_Lambda([this] { return FilterByBuildVersion->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::BuildBranch)
			.FillWidth(0.2f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::BuildBranch)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeBranchColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("BranchColumn", "Build Branch"))
					.ColorAndOpacity_Lambda([this] { return FilterByBranch->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

#if 0
			+ SHeaderRow::Column(FTraceListColumns::Changelist)
			.FillWidth(0.1f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::Changelist)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeChangelistColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ChangelistColumn", "CL"))
					.ColorAndOpacity_Lambda([this] { return FilterByChangelist->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]
#endif

			+ SHeaderRow::Column(FTraceListColumns::EngineVersion)
			.FillWidth(0.1f)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::EngineVersion)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeEngineVersionColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EngineVersionColumn", "UE"))
					.ColorAndOpacity_Lambda([this] { return FilterByEngineVersion->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::Size)
			.FixedWidth(100.0f)
			.HAlignHeader(HAlign_Right)
			.HAlignCell(HAlign_Right)
			.InitialSortMode(EColumnSortMode::Descending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::Size)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeSizeColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(24.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SizeColumn", "File Size"))
					.ColorAndOpacity_Lambda([this] { return FilterBySize->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]

			+ SHeaderRow::Column(FTraceListColumns::Status)
			.FixedWidth(60.0f)
			.HAlignHeader(HAlign_Right)
			.HAlignCell(HAlign_Right)
			.InitialSortMode(EColumnSortMode::Ascending)
			.SortMode(this, &STraceStoreWindow::GetSortModeForColumn, FTraceListColumns::Status)
			.OnSort(this, &STraceStoreWindow::OnSortModeChanged)
			.OnGetMenuContent(this, &STraceStoreWindow::MakeStatusColumnHeaderMenu)
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(24.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("StatusColumn", "Status"))
					.ColorAndOpacity_Lambda([this] { return FilterByStatus->IsEmpty() ? FLinearColor(0.5f, 0.5f, 0.5f, 1.0f) : FLinearColor(0.3f, 0.75f, 1.0f, 1.0f); })
				]
			]
		);

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STraceStoreWindow::ConstructLoadPanel()
{
	TSharedRef<SWidget> Widget = SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.FillWidth(1.0f)
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		[
			ConstructAutoStartPanel()
		]
	]

	+ SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
		.IsEnabled(this, &STraceStoreWindow::Open_IsEnabled)
		.OnClicked(this, &STraceStoreWindow::Open_OnClicked)
		.ToolTipText(LOCTEXT("OpenButtonTooltip", "Start analysis for selected trace session."))
		.AddMetaData(FDriverMetaData::Id("OpenTraceButton"))
		.ContentPadding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
		.Content()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DialogButtonText"))
				.Justification(ETextJustify::Center)
				.Text(LOCTEXT("OpenButtonText", "Open Trace"))
			]
		]
	]

	+ SHorizontalBox::Slot()
	.Padding(FMargin(6.0f, 0.0f, 0.0f, 0.0f))
	.AutoWidth()
	[
		SNew(SComboButton)
		.ToolTipText(LOCTEXT("MRU_Tooltip", "Open a trace file or choose a trace session."))
		.OnGetMenuContent(this, &STraceStoreWindow::MakeTraceListMenu)
		.HasDownArrow(true)
	];

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STraceStoreWindow::ConstructTraceStoreDirectoryPanel()
{
	TSharedRef<SWidget> Widget =

		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 8.0f)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &STraceStoreWindow::VisibleIfNotConnected)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Raw(this, &STraceStoreWindow::GetConnectionStatusTooltip)
				.ColorAndOpacity(EStyleColor::Error)
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("ManageStoreSettingsTooltip", "Manage store settings."))
				.OnClicked(this, &STraceStoreWindow::StoreSettingsArea_Toggle)
				[
					SNew(SImage)
					.Image_Raw(this, &STraceStoreWindow::StoreSettingsToggle_Icon)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image_Raw(this, &STraceStoreWindow::GetConnectionStatusIcon)
				.ToolTip(SNew(SToolTip)
				.Text_Raw(this, &STraceStoreWindow::GetConnectionStatusTooltip))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("StoreHostText", "Store Host:"))
			]

			+ SHorizontalBox::Slot()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SAssignNew(StoreHostTextBox, SEditableTextBox)
				.IsReadOnly(true)
				.BackgroundColor(FSlateColor(EStyleColor::Background))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(this, &STraceStoreWindow::VisibleIfConnected)
				.Text(LOCTEXT("TraceStoreDirText", "Directory:"))
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(StoreDirTextBox, SEditableTextBox)
				.Visibility(this, &STraceStoreWindow::VisibleIfConnected)
				.IsReadOnly(true)
				.BackgroundColor(FSlateColor(EStyleColor::Background))
				.Text(this, &STraceStoreWindow::GetTraceStoreDirectory)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.Visibility(this, &STraceStoreWindow::VisibleIfConnected)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
				.ToolTipText(LOCTEXT("ExploreTraceStoreDirButtonToolTip", "Explores the Trace Store Directory."))
				.OnClicked(this, &STraceStoreWindow::ExploreTraceStoreDirectory_OnClicked)
				.AddMetaData(FDriverMetaData::Id("ExploreTraceStoreDirButton"))
				.IsEnabled(this, &STraceStoreWindow::CanChangeStoreSettings)
				[
					SNew(SImage)
					.Image(FInsightsCoreStyle::Get().GetBrush("Icons.FolderExplore"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.MaxHeight(400)
		.Padding(0.0f, 8.0f)
		.HAlign(HAlign_Left)
		[
			SAssignNew(StoreSettingsArea, SScrollBox)
			.Orientation(Orient_Vertical)
			.Visibility(EVisibility::Collapsed)

			+ SScrollBox::Slot()
			[
				SNew(SVerticalBox)
				.Visibility(this, &STraceStoreWindow::VisibleIfNotConnected)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NotConnected", "Not connected to a Trace Server!"))
					.ColorAndOpacity(EStyleColor::Warning)
				]
			]

			+ SScrollBox::Slot()
			[
				SNew(SVerticalBox)
				.Visibility(this, &STraceStoreWindow::VisibleIfConnected)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("StoreDirLabel", "Trace Store Directory (new traces will be stored here):"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f)
				[
					SAssignNew(StoreDirListView, SListView<TSharedPtr<FTraceDirectoryModel>>)
					.ListItemsSource(&StoreDirectoryModel)
					.OnGenerateRow(this, &STraceStoreWindow::TraceDirs_OnGenerateRow)
					.SelectionMode(ESelectionMode::None)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("WatchDirsLabel", "Additional directories to monitor for traces:"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f)
				[
					SAssignNew(WatchDirsListView, SListView<TSharedPtr<FTraceDirectoryModel>>)
					.ListItemsSource(&WatchDirectoriesModel)
					.OnGenerateRow(this, &STraceStoreWindow::TraceDirs_OnGenerateRow)
					.SelectionMode(ESelectionMode::None)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 4.0f)
				.HAlign(HAlign_Left)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
					.ToolTipText(LOCTEXT("WatchDirsAddTooltip", "Adds an additional directory to monitor for traces."))
					.IsEnabled(this, &STraceStoreWindow::CanChangeStoreSettings)
					.OnClicked(this, &STraceStoreWindow::AddWatchDir_Clicked)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FInsightsFrontendStyle::Get().GetBrush("Icons.AddWatchDir"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]

						+ SHorizontalBox::Slot()
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(FText::FromStringView(TEXTVIEW("Add Directory...")))
						]
					]
				]
			]
		]
		;

	return Widget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> STraceStoreWindow::ConstructAutoStartPanel()
{
	TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox)

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(0.0f, 0.0f, 0.0f, 0.0f)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SCheckBox)
		.ToolTipText(LOCTEXT("AutoStart_Tooltip", "Enable auto-start analysis for LIVE trace sessions."))
		.IsChecked(this, &STraceStoreWindow::AutoStart_IsChecked)
		.OnCheckStateChanged(this, &STraceStoreWindow::AutoStart_OnCheckStateChanged)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AutoStart_Text", "Auto-start (LIVE)"))
		]
	]

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(6.0f, 0.0f, 0.0f, 0.0f)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		.MaxDesiredWidth(200.0f)
		[
			SAssignNew(AutoStartPlatformFilter, SSearchBox)
			.InitialText(FText::FromString(GetSettings().GetAutoStartAnalysisPlatform()))
			.OnTextCommitted(this, &STraceStoreWindow::AutoStartPlatformFilterBox_OnValueCommitted)
			.HintText(LOCTEXT("AutoStartPlatformFilter_Hint", "Platform"))
			.ToolTipText(LOCTEXT("AutoStartPlatformFilter_Tooltip", "Type here to specify the Platform filter.\nAuto-start analysis will be enabled only for live trace sessions with this specified Platform."))
		]
	]

	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(6.0f, 0.0f, 0.0f, 0.0f)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	[
		SNew(SBox)
		.MaxDesiredWidth(200.0f)
		[
			SAssignNew(AutoStartAppNameFilter, SSearchBox)
			.InitialText(FText::FromString(GetSettings().GetAutoStartAnalysisAppName()))
			.OnTextCommitted(this, &STraceStoreWindow::AutoStartAppNameFilterBox_OnValueCommitted)
			.HintText(LOCTEXT("AutoStartAppNameFilter_Hint", "AppName"))
			.ToolTipText(LOCTEXT("AutoStartAppNameFilter_Tooltip", "Type here to specify the AppName filter.\nAuto-start analysis will be enabled only for live trace sessions with this specified AppName."))
		]
	];

	Box->AddSlot()
		.AutoWidth()
		.Padding(6.0f, 0.0f, 0.0f, 0.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		];

	Box->AddSlot()
		.AutoWidth()
		.Padding(6.0f, 0.0f, 0.0f, 0.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("AutoConnect_Tooltip", "Signal to UE applications to auto-connect with local trace server and start tracing if Insights is running."))
			.IsChecked(this, &STraceStoreWindow::AutoConnect_IsChecked)
			.OnCheckStateChanged(this, &STraceStoreWindow::AutoConnect_OnCheckStateChanged)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AutoConnect_Text", "Auto-connect"))
			]
		];

	return Box;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> STraceStoreWindow::TraceList_OnGenerateRow(TSharedPtr<FTraceViewModel> InTrace, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STraceListRow, InTrace, SharedThis(this), OwnerTable);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceStoreWindow::GetConnectionStatusTooltip() const
{
	static FText Connected    = LOCTEXT("Connected",    "Connected to the trace server.\nServer version: {0}\nRecorder port: {1}, Store port: {2}");
	static FText NotConnected = LOCTEXT("NoConnection", "Unable to connect to trace server.");
	static FText Connecting   = LOCTEXT("Connecting",   "Trying to connect to trace server.");
	static FText Disconnected = LOCTEXT("Disconnected", "Connection to trace server has been lost. Attempting to reconnect in {0} seconds.");

	const FStoreBrowser::EConnectionStatus Status = StoreBrowser->GetConnectionStatus();

	switch (Status)
	{
		case FStoreBrowser::EConnectionStatus::Connected:
		{
			StoreBrowser->LockSettings();
			FText Version = FText::FromString(StoreBrowser->GetVersion());
			const uint32 RecorderPort = StoreBrowser->GetRecorderPort();
			const uint32 StorePort = StoreBrowser->GetStorePort();
			StoreBrowser->UnlockSettings();
			return FText::Format(Connected,
				Version,
				FText::AsNumber(RecorderPort, &FNumberFormattingOptions::DefaultNoGrouping()),
				FText::AsNumber(StorePort,  &FNumberFormattingOptions::DefaultNoGrouping())
			);
		}

		case FStoreBrowser::EConnectionStatus::NoConnection:
			return NotConnected;

		case FStoreBrowser::EConnectionStatus::Connecting:
			return Connecting;

		default:
			return FText::Format(Disconnected, FText::AsNumber(static_cast<uint32>(Status)));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* STraceStoreWindow::GetConnectionStatusIcon() const
{
	const FStoreBrowser::EConnectionStatus Status = StoreBrowser->GetConnectionStatus();
	return (Status == FStoreBrowser::EConnectionStatus::Connected) ?
		FInsightsFrontendStyle::Get().GetBrush("Icons.Online") :
		FInsightsFrontendStyle::Get().GetBrush("Icons.Offline");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility STraceStoreWindow::VisibleIfNotConnected() const
{
	const FStoreBrowser::EConnectionStatus Status = StoreBrowser->GetConnectionStatus();
	return (Status == FStoreBrowser::EConnectionStatus::Connected) ?
		EVisibility::Collapsed :
		EVisibility::Visible;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility STraceStoreWindow::VisibleIfConnected() const
{
	const FStoreBrowser::EConnectionStatus Status = StoreBrowser->GetConnectionStatus();
	return (Status == FStoreBrowser::EConnectionStatus::Connected) ?
		EVisibility::Visible :
		EVisibility::Collapsed;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility STraceStoreWindow::HiddenIfNotConnected() const
{
	const FStoreBrowser::EConnectionStatus Status = StoreBrowser->GetConnectionStatus();
	return (Status == FStoreBrowser::EConnectionStatus::Connected) ?
		EVisibility::Visible :
		EVisibility::Hidden;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<SWidget> STraceStoreWindow::TraceList_GetMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("Misc");
	{
		{
			FMenuEntryParams MenuEntry;
			MenuEntry.LabelOverride = LOCTEXT("ContextMenu_Rename", "Rename...");
			MenuEntry.InputBindingOverride = LOCTEXT("ContextMenu_Rename_InputBinding", "F2");
			MenuEntry.ToolTipOverride = LOCTEXT("ContextMenu_Rename_ToolTip", "Starts renaming of the selected trace file.");
			MenuEntry.IconOverride = FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.Rename");
			MenuEntry.DirectActions = FUIAction(
				FExecuteAction::CreateSP(this, &STraceStoreWindow::RenameSelectedTrace),
				FCanExecuteAction::CreateSP(this, &STraceStoreWindow::CanRenameSelectedTrace));
			MenuEntry.UserInterfaceActionType = EUserInterfaceActionType::Button;
			MenuBuilder.AddMenuEntry(MenuEntry);
		}
		{
			FMenuEntryParams MenuEntry;
			MenuEntry.LabelOverride = LOCTEXT("ContextMenu_Delete", "Delete");
			MenuEntry.InputBindingOverride = LOCTEXT("ContextMenu_Delete_InputBinding", "Del");
			MenuEntry.ToolTipOverride = LOCTEXT("ContextMenu_Delete_ToolTip", "Deletes the selected trace files.");
			MenuEntry.IconOverride = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Delete");
			MenuEntry.DirectActions = FUIAction(
				FExecuteAction::CreateSP(this, &STraceStoreWindow::DeleteSelectedTraces),
				FCanExecuteAction::CreateSP(this, &STraceStoreWindow::CanDeleteSelectedTraces));
			MenuEntry.UserInterfaceActionType = EUserInterfaceActionType::Button;
			MenuBuilder.AddMenuEntry(MenuEntry);
		}
		MenuBuilder.AddSeparator();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_CopyTraceId", "Copy Trace Id"),
			LOCTEXT("ContextMenu_CopyTraceId_ToolTip", "Copies the unique id of the selected trace session."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
			FUIAction(
				FExecuteAction::CreateSP(this, &STraceStoreWindow::CopyTraceId),
				FCanExecuteAction::CreateSP(this, &STraceStoreWindow::CanCopyTraceId)),
			NAME_None,
			EUserInterfaceActionType::Button);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_CopyUri", "Copy Full Path"),
			LOCTEXT("ContextMenu_CopyUri_ToolTip", "Copies the full path of the selected trace file."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
			FUIAction(
				FExecuteAction::CreateSP(this, &STraceStoreWindow::CopyFullPath),
				FCanExecuteAction::CreateSP(this, &STraceStoreWindow::CanCopyFullPath)),
			NAME_None,
			EUserInterfaceActionType::Button);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ContextMenu_OpenContainingFolder", "Open Containing Folder"),
			LOCTEXT("ContextMenu_OpenContainingFolder_ToolTip", "Opens the containing folder of the selected trace file."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
			FUIAction(
				FExecuteAction::CreateSP(this, &STraceStoreWindow::OpenContainingFolder),
				FCanExecuteAction::CreateSP(this, &STraceStoreWindow::CanOpenContainingFolder)),
			NAME_None,
			EUserInterfaceActionType::Button);
	}

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::CanRenameSelectedTrace() const
{
	if (!CanChangeStoreSettings())
	{
		return false;
	}
	TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
	return SelectedTrace.IsValid()
		&& SelectedTrace->HasValidId()
		&& !SelectedTrace->IsLive();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::RenameSelectedTrace()
{
	FSlateApplication::Get().CloseToolTip();

	if (!CanRenameSelectedTrace())
	{
		return;
	}

	TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
	SelectedTrace->StartRenaming();

	TSharedPtr<SEditableTextBox> RenameTextBox = SelectedTrace->GetRenameTextBox();
	if (RenameTextBox.IsValid())
	{
		FSlateApplication::Get().SetKeyboardFocus(RenameTextBox.ToSharedRef(), EFocusCause::SetDirectly);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::CanDeleteSelectedTraces() const
{
	if (!CanChangeStoreSettings() ||
		TraceListView->GetNumItemsSelected() == 0)
	{
		return false;
	}

	TArray<TSharedPtr<FTraceViewModel>> SelectedTraces = TraceListView->GetSelectedItems();
	for (const TSharedPtr<FTraceViewModel>& SelectedTrace : SelectedTraces)
	{
		if (SelectedTrace->HasValidId() && !SelectedTrace->IsLive())
		{
			return true;
		}
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::DeleteSelectedTraces()
{
	FSlateApplication::Get().CloseToolTip();

	if (!CanDeleteSelectedTraces())
	{
		return;
	}

	TArray<TSharedPtr<FTraceViewModel>> TracesToDelete = TraceListView->GetSelectedItems();
	// Filter the traces that can actually be deleted :
	TracesToDelete.RemoveAll([](const TSharedPtr<FTraceViewModel>& InTrace) { return !InTrace->HasValidId() || InTrace->IsLive(); });
	if (TracesToDelete.Num() == 0)
	{
		return;
	}

	if (bIsDeleteTraceConfirmWindowVisible)
	{
		//TODO: Make a custom OkCancel modal dialog. See FSlateApplication::Get().AddModalWindow(..).
		FText Title = LOCTEXT("ConfirmToDeleteTraceFile_Title", "Unreal Insights");
		TStringBuilder<2048> TraceFilesToDelete;
		for (int32 TraceIndex = 0; TraceIndex < TracesToDelete.Num() && TraceIndex < 3; ++TraceIndex)
		{
			TraceFilesToDelete.Append(TracesToDelete[TraceIndex]->GetUri());
			TraceFilesToDelete.Append(TEXT("\n"));
		}
		if (TracesToDelete.Num() > 3)
		{
			TraceFilesToDelete.Append(TEXT("...\n"));
		}
		FText ConfirmMessage = FText::Format(LOCTEXT("ConfirmToDeleteTraceFile", "You are about to delete {0} trace {0}|plural(one=file,other=files):\n\n{1}\nPress OK to continue."),
			TracesToDelete.Num(), FText::FromStringView(TraceFilesToDelete.ToView()));
		EAppReturnType::Type OkToDelete = FMessageDialog::Open(EAppMsgType::OkCancel, ConfirmMessage, Title);
		if (OkToDelete == EAppReturnType::Cancel)
		{
			return;
		}
	}

	// Find an unselected item (close to last selected one).
	int32 TraceIndexToSelect = -1;
	for (int32 TraceIndex = 0; TraceIndex < TracesToDelete.Num(); ++TraceIndex)
	{
		FTraceViewModel* TraceVM = TracesToDelete[TraceIndex].Get();
		int32 FilteredTraceIndex = FilteredTraceViewModels.IndexOfByPredicate([TraceVM](const TSharedPtr<FTraceViewModel>& VM) { return VM.Get() == TraceVM; });
		if (FilteredTraceIndex + 1 >= 0 &&
			FilteredTraceIndex + 1 < FilteredTraceViewModels.Num() &&
			!TraceListView->IsItemSelected(FilteredTraceViewModels[FilteredTraceIndex + 1]))
		{
			if (FilteredTraceIndex + 1> TraceIndexToSelect)
			{
				TraceIndexToSelect = FilteredTraceIndex + 1;
			}
		}
		else
		if (FilteredTraceIndex - 1 >= 0 &&
			FilteredTraceIndex - 1 < FilteredTraceViewModels.Num() &&
			!TraceListView->IsItemSelected(FilteredTraceViewModels[FilteredTraceIndex - 1]))
		{
			if (FilteredTraceIndex - 1 > TraceIndexToSelect)
			{
				TraceIndexToSelect = FilteredTraceIndex - 1;
			}
		}
	}
	TSharedPtr<FTraceViewModel> TraceToSelect = (TraceIndexToSelect >= 0) ? FilteredTraceViewModels[TraceIndexToSelect] : nullptr;

	// Delete traces.
	int32 NumDeletedTraces = 0;
	for (int32 TraceIndex = 0; TraceIndex < TracesToDelete.Num(); ++TraceIndex)
	{
		const TSharedPtr<FTraceViewModel>& TraceToDelete = TracesToDelete[TraceIndex];
		if (DeleteTrace(TraceToDelete))
		{
			++NumDeletedTraces;

			TraceListView->SetItemSelection(TraceToDelete, false);

			FilteredTraceViewModels.Remove(TraceToDelete);

			TraceViewModels.Remove(TraceToDelete);
			TraceViewModelMap.Remove(TraceToDelete->GetId());
		}
	}

	if (NumDeletedTraces == TracesToDelete.Num())
	{
		FText Message = FText::Format(LOCTEXT("DeleteSuccessFmt", "Successfully deleted {0} trace {0}|plural(one=file,other=files)."), NumDeletedTraces);
		ShowSuccessMessage(Message);

		// Set new selection.
		if (TraceToSelect.IsValid())
		{
			TraceListView->SetItemSelection(TraceToSelect, true);
			bIsUserSelectedTrace = true;
		}
	}
	else
	{
		FText Message = FText::Format(LOCTEXT("FailedToDeleteAllTracesFmt", "Deleted {0} trace {0}|plural(one=file,other=files). Failed to delete {1} trace {1}|plural(one=file,other=files)!"),
			NumDeletedTraces, TracesToDelete.Num() - NumDeletedTraces);
		ShowFailMessage(Message);
	}

	OnTraceListChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::DeleteTrace(const TSharedPtr<FTraceViewModel>& TraceToDelete)
{
	FString TraceName = TraceToDelete->GetName();
	UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Deleting \"%ls\"...", *TraceName);

	if (TraceToDelete->IsLive())
	{
		FText Message = FText::Format(LOCTEXT("CannotDeleteLiveTraceFmt", "Cannot delete a live trace (\"{0}\")!"), FText::FromString(TraceName));
		ShowFailMessage(Message);
		return false;
	}

	FString TraceFile = TraceToDelete->GetUri();
	if (!FPaths::FileExists(TraceFile) || !IFileManager::Get().Delete(*TraceFile))
	{
		UE_LOGF(LogInsightsFrontend, Warning, "[TraceStore] Failed to delete trace file (\"%ls\")!", *TraceFile);

		FText Message = FText::Format(LOCTEXT("DeleteFailFmt", "Failed to delete \"{0}\"!"), FText::FromString(TraceName));
		ShowFailMessage(Message);
		return false;
	}

	UE_LOGF(LogInsightsFrontend, Verbose, "[TraceStore] Deleted utrace file (\"%ls\").", *TraceFile);

	FString CacheFile = FPaths::ChangeExtension(TraceFile, TEXT("ucache"));
	if (FPaths::FileExists(CacheFile))
	{
		if (IFileManager::Get().Delete(*CacheFile))
		{
			UE_LOGF(LogInsightsFrontend, Verbose, "[TraceStore] Deleted ucache file (\"%ls\").", *CacheFile);
		}
	}

	// Also delete the metadata ini file (<trace>.utrace.ini) if present.
	const FString SidecarFile = TraceFile + TEXT(".ini");
	if (FPaths::FileExists(SidecarFile))
	{
		if (IFileManager::Get().Delete(*SidecarFile))
		{
			UE_LOGF(LogInsightsFrontend, Verbose, "[TraceStore] Deleted sidecar ini file (\"%ls\").", *SidecarFile);
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::CanCopyTraceId() const
{
	TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
	return SelectedTrace.IsValid() && SelectedTrace->HasValidId();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::CopyTraceId()
{
	if (CanCopyTraceId())
	{
		TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
		FString ClipboardText = FString::Printf(TEXT("0x%X"), SelectedTrace->GetId());
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::CanCopyFullPath() const
{
	TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
	return SelectedTrace.IsValid() && SelectedTrace->HasValidId();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::CopyFullPath()
{
	if (CanCopyFullPath())
	{
		TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
		FPlatformApplicationMisc::ClipboardCopy(*SelectedTrace->GetUri());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::CanOpenContainingFolder() const
{
	if (!CanChangeStoreSettings())
	{
		return false;
	}
	TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
	return SelectedTrace.IsValid() && SelectedTrace->HasValidId();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OpenContainingFolder()
{
	FSlateApplication::Get().CloseToolTip();

	if (CanOpenContainingFolder())
	{
		TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
		FPlatformProcess::ExploreFolder(*SelectedTrace->GetUri());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::HasAnyLiveTrace() const
{
	return TraceViewModels.FindByPredicate([](const TSharedPtr<FTraceViewModel>& Trace) { return Trace->IsLive(); }) != nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::ShowSplashScreenOverlay()
{
	SplashScreenOverlayFadeTime = 3.5f;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::TickSplashScreenOverlay(const float InDeltaTime)
{
	if (SplashScreenOverlayFadeTime > 0.0f)
	{
		SplashScreenOverlayFadeTime = FMath::Max(0.0f, SplashScreenOverlayFadeTime - InDeltaTime);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

float STraceStoreWindow::SplashScreenOverlayOpacity() const
{
	constexpr float FadeInStartTime = 3.5f;
	constexpr float FadeInEndTime = 3.0f;
	constexpr float FadeOutStartTime = 1.0f;
	constexpr float FadeOutEndTime = 0.0f;

	const float Opacity =
		SplashScreenOverlayFadeTime > FadeInStartTime ? 0.0f :
		SplashScreenOverlayFadeTime > FadeInEndTime ? 1.0f - (SplashScreenOverlayFadeTime - FadeInEndTime) / (FadeInStartTime - FadeInEndTime) :
		SplashScreenOverlayFadeTime > FadeOutStartTime ? 1.0f :
		SplashScreenOverlayFadeTime > FadeOutEndTime ? (SplashScreenOverlayFadeTime - FadeOutEndTime) / (FadeOutStartTime - FadeOutEndTime) : 0.0f;

	return Opacity;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EVisibility STraceStoreWindow::SplashScreenOverlay_Visibility() const
{
	return SplashScreenOverlayFadeTime > 0.0f ? EVisibility::Visible : EVisibility::Collapsed;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STraceStoreWindow::SplashScreenOverlay_ColorAndOpacity() const
{
	return FSlateColor(FLinearColor(0.7f, 0.7f, 0.7f, SplashScreenOverlayOpacity()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STraceStoreWindow::SplashScreenOverlay_TextColorAndOpacity() const
{
	return FSlateColor(FLinearColor(0.8f, 0.8f, 0.8f, SplashScreenOverlayOpacity()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceStoreWindow::GetSplashScreenOverlayText() const
{
	return FText::Format(LOCTEXT("StartAnalysis", "Starting analysis...\n{0}"), FText::FromString(SplashScreenOverlayTraceFile));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::RefreshTraces_OnClicked()
{
	RefreshTraceList();
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FSlateColor STraceStoreWindow::GetColorByPath(const FString& Uri)
{
	const FStringView UriBase = FPathViews::GetPath(Uri);
	const TSharedPtr<FTraceDirectoryModel>* Dir = WatchDirectoriesModel.FindByPredicate([&](const TSharedPtr<FTraceDirectoryModel>& Dir)
	{
		return FPathViews::Equals(UriBase, Dir->Path);
	});
	if (Dir)
	{
		return FAppStyle::Get().GetSlateColor((*Dir)->Color);
	}
	// If this is default trace store directory, use foreground
	return FSlateColor::UseForeground();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::RefreshTraceList()
{
	FStopwatch StopwatchTotal;
	StopwatchTotal.Start();

	int32 AddedTraces = 0;
	int32 RemovedTraces = 0;
	int32 UpdatedTraces = 0;
	bool bSettingsChanged = false;

	{
		StoreBrowser->LockSettings();

		const uint32 NewSettingsChangeSerial = StoreBrowser->GetSettingsChangeSerial();
		if (NewSettingsChangeSerial != SettingsChangeSerial)
		{
			SettingsChangeSerial = NewSettingsChangeSerial;

			// Add remote server controls. It's not possible to change server
			// address on the fly so we can expect that there cannot be more than
			// two entries (the local and possibly a currently connected remote server)
			if (!StoreBrowser->GetHost().Equals(TEXT("127.0.0.1")) && ServerControls.Num() == 1)
			{
				 ServerControls.Emplace(*StoreBrowser->GetHost(), StoreBrowser->GetStorePort(), FAppStyle::Get().GetStyleSetName());
			}

			// Update the host text
			if (StoreHostTextBox)
			{
				StoreHostTextBox->SetText(FText::FromString(StoreBrowser->GetHost()));
			}

			// Update the store text box
			if (StoreDirTextBox)
			{
				StoreDirTextBox->SetText(FText::FromString(StoreBrowser->GetStoreDirectory()));
			}

			// Update store directory model
			StoreDirectoryModel.Empty(1);
			StoreDirectoryModel.Push(MakeShared<FTraceDirectoryModel>(
				FString(StoreBrowser->GetStoreDirectory()),
				NAME_None,
				ETraceDirOperations::ModifyStore|ETraceDirOperations::Explore
			));
			if (StoreDirListView)
			{
				StoreDirListView->RequestListRefresh();
			}

			// Update additional monitored directories model
			static const FName DirColor[] =
			{
				FName("Colors.AccentBlue"),
				FName("Colors.AccentGreen"),
				FName("Colors.AccentYellow"),
				FName("Colors.AccentOrange"),
				FName("Colors.AccentPurple"),
				FName("Colors.AccentPink")
			};
			int32 ColorIdx = 0;
			WatchDirectoriesModel.Empty();
			for (const auto& Dir : StoreBrowser->GetWatchDirectories())
			{
				WatchDirectoriesModel.Emplace(MakeShared<FTraceDirectoryModel>(
					FString(Dir),
					DirColor[ColorIdx],
					ETraceDirOperations::Delete|ETraceDirOperations::Explore
				));
				ColorIdx = FMath::WrapExclusive(++ColorIdx, (int32)0, int32(UE_ARRAY_COUNT(DirColor)));
			}
			if (WatchDirsListView)
			{
				WatchDirsListView->RequestListRefresh();
			}

			bSettingsChanged = true;
		}

		StoreBrowser->UnlockSettings();
		StoreBrowser->LockTraces();

		const uint32 NewTracesChangeSerial = StoreBrowser->GetTracesChangeSerial();
		if (NewTracesChangeSerial != TracesChangeSerial || bSettingsChanged)
		{
			TracesChangeSerial = NewTracesChangeSerial;
			//UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Syncing the trace list with StoreBrowser...");

			const TArray<TSharedPtr<FStoreBrowserTraceInfo>>& InTraces = StoreBrowser->GetTraces();
			const TMap<uint32, TSharedPtr<FStoreBrowserTraceInfo>>& InTraceMap = StoreBrowser->GetTraceMap();

			// Check for removed traces.
			{
				int32 TraceViewModelCount = TraceViewModels.Num();
				for (int32 TraceIndex = 0; TraceIndex < TraceViewModelCount; ++TraceIndex)
				{
					FTraceViewModel& Trace = *TraceViewModels[TraceIndex];
					const TSharedPtr<FStoreBrowserTraceInfo>* InTracePtrPtr = InTraceMap.Find(Trace.GetId());
					if (!InTracePtrPtr)
					{
						// This trace was removed.
						RemovedTraces++;
						TraceViewModels.RemoveAtSwap(TraceIndex);
						TraceViewModelMap.Remove(Trace.GetId());
						TraceIndex--;
						TraceViewModelCount--;
					}
				}
			}

			// Check for added traces and for updated traces.
			for (const TSharedPtr<FStoreBrowserTraceInfo>& InTracePtr : InTraces)
			{
				const FStoreBrowserTraceInfo& SourceTrace = *InTracePtr;
				TSharedPtr<FTraceViewModel>* TracePtrPtr = TraceViewModelMap.Find(SourceTrace.TraceId);
				if (TracePtrPtr)
				{
					FTraceViewModel& Trace = **TracePtrPtr;
					if (Trace.GetChangeSerial() != SourceTrace.ChangeSerial || bSettingsChanged)
					{
						// This trace was updated or settings updated
						UpdatedTraces++;
						UpdateTrace(Trace, SourceTrace);
					}
				}
				else
				{
					// This trace was added.
					AddedTraces++;
					TSharedPtr<FTraceViewModel> TracePtr = MakeShared<FTraceViewModel>();
					TracePtr->SetId(SourceTrace.TraceId);
					UpdateTrace(*TracePtr, SourceTrace);
					TraceViewModels.Add(TracePtr);
					TraceViewModelMap.Add(TracePtr->GetId(), TracePtr);
				}
			}
		}

		StoreBrowser->UnlockTraces();
	}

	if (AddedTraces > 0 || RemovedTraces > 0)
	{
		// If we have new or removed traces we need to rebuild the list view.
		OnTraceListChanged();
	}

	StopwatchTotal.Stop();
	const double Duration = StopwatchTotal.GetAccumulatedTime();
	if ((Duration > 0.0001) && (UpdatedTraces > 0 || AddedTraces > 0 || RemovedTraces > 0))
	{
		UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] The trace list refreshed in %.0f ms (%d traces : %d updated, %d added, %d removed).",
			Duration * 1000.0, TraceViewModels.Num(), UpdatedTraces, AddedTraces, RemovedTraces);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::IsConnected() const
{
	return StoreBrowser->GetConnectionStatus() == FStoreBrowser::EConnectionStatus::Connected;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::UpdateTrace(FTraceViewModel& InOutTrace, const FStoreBrowserTraceInfo& InSourceTrace)
{
	check(InOutTrace.GetId() == InSourceTrace.TraceId);

	InOutTrace.SetChangeSerial(InSourceTrace.ChangeSerial);

	InOutTrace.SetName(InSourceTrace.Name);
	InOutTrace.SetUri(InSourceTrace.Uri);
	InOutTrace.SetDirectoryColor(GetColorByPath(InSourceTrace.Uri));

	InOutTrace.SetTimestamp(InSourceTrace.Timestamp);
	InOutTrace.SetSize(InSourceTrace.Size);

	InOutTrace.SetIsLive(InSourceTrace.bIsLive);
	InOutTrace.SetIpAddress(InSourceTrace.IpAddress);

	// Is metadata updated?
	if (!InOutTrace.IsMetadataUpdated() && InSourceTrace.MetadataUpdateCount == 0)
	{
		InOutTrace.SetIsMetadataUpdated(true);
		InOutTrace.SetPlatform(InSourceTrace.Platform);
		if (!InSourceTrace.ProjectName.IsEmpty())
		{
			InOutTrace.SetAppName(InSourceTrace.ProjectName);
		}
		else
		{
			InOutTrace.SetAppName(InSourceTrace.AppName);
		}
		InOutTrace.SetBuildConfiguration(InSourceTrace.ConfigurationType);
		InOutTrace.SetBuildTarget(InSourceTrace.TargetType);
		InOutTrace.SetBuildVersion(InSourceTrace.BuildVersion);
		InOutTrace.SetBuildBranch(InSourceTrace.Branch);
		InOutTrace.SetChangelist(InSourceTrace.Changelist);
		InOutTrace.SetEngineVersion(InSourceTrace.EngineVersion);
		InOutTrace.SetCommandLine(InSourceTrace.CommandLine);
		InOutTrace.SetVFSPaths(InSourceTrace.VFSPaths);
	}

	const FInsightsFrontendSettings& Settings = GetSettings();

	// Auto start analysis for a live trace session.
	if (InOutTrace.IsLive() &&
		InOutTrace.IsMetadataUpdated() &&
		Settings.IsAutoStartAnalysisEnabled() && // is auto start enabled?
		!AutoStartedSessions.Contains(InOutTrace.GetId())) // is not already auto-started?
	{
		const FString& AutoStartPlatformFilterStr = Settings.GetAutoStartAnalysisPlatform();
		const FString& AutoStartAppNameFilterStr = Settings.GetAutoStartAnalysisAppName();

		// matches filter?
		if ((AutoStartPlatformFilterStr.IsEmpty() || FCString::Strcmp(*AutoStartPlatformFilterStr, *InOutTrace.GetPlatform()) == 0) &&
			(AutoStartAppNameFilterStr.IsEmpty() || FCString::Strcmp(*AutoStartAppNameFilterStr, *InOutTrace.GetAppName()) == 0) &&
			(AutoStartConfigurationTypeFilter == EBuildConfiguration::Unknown || AutoStartConfigurationTypeFilter == InOutTrace.GetBuildConfiguration()) &&
			(AutoStartTargetTypeFilter == EBuildTargetType::Unknown || AutoStartTargetTypeFilter == InOutTrace.GetBuildTarget()))
		{
			UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Auto starting analysis for trace with id 0x%08X...", InOutTrace.GetId());
			AutoStartedSessions.Add(InOutTrace.GetId());
			OpenTraceSession(InOutTrace.GetId());
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OnTraceListChanged()
{
	UpdateFiltering();
	UpdateSorting();
	UpdateTraceListView();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::UpdateTraceListView()
{
	if (!TraceListView)
	{
		return;
	}

	TArray<TSharedPtr<FTraceViewModel>> NewSelectedTraces;
	if (bIsUserSelectedTrace)
	{
		// Identify the previously selected traces (if still available) to ensure selection remains unchanged.
		TArray<TSharedPtr<FTraceViewModel>> SelectedTraces = TraceListView->GetSelectedItems();
		for (const TSharedPtr<FTraceViewModel>& SelectedTrace : SelectedTraces)
		{
			TSharedPtr<FTraceViewModel>* FoundNewTrace = TraceViewModelMap.Find(SelectedTrace->GetId());
			if (!FoundNewTrace)
			{
				FoundNewTrace = TraceViewModels.FindByPredicate([SelectedTrace](const TSharedPtr<FTraceViewModel>& Trace) { return Trace->HasSameUri(*SelectedTrace); });
			}
			if (!FoundNewTrace)
			{
				FoundNewTrace = TraceViewModels.FindByPredicate([SelectedTrace](const TSharedPtr<FTraceViewModel>& Trace) { return Trace->HasSameName(*SelectedTrace); });
			}
			if (FoundNewTrace)
			{
				NewSelectedTraces.Add(*FoundNewTrace);
			}
		}
	}

	double DistanceFromTop = TraceListView->GetScrollDistance().Y;
	double DistanceFromBottom = TraceListView->GetScrollDistanceRemaining().Y;

	TraceListView->RebuildList();

	// If no selection...
	if (NewSelectedTraces.Num() == 0 && FilteredTraceViewModels.Num() > 0)
	{
		if ((SortColumn == FTraceListColumns::Date && SortMode == EColumnSortMode::Ascending) ||
			(SortColumn == FTraceListColumns::Status && SortMode == EColumnSortMode::Ascending))
		{
			// Auto select the last (newest) trace.
			NewSelectedTraces.Add(FilteredTraceViewModels.Last());
			DistanceFromTop = 1.0;
			DistanceFromBottom = 0.0; // scroll to bottom
		}
		else
		{
			// Auto select the first trace.
			NewSelectedTraces.Add(FilteredTraceViewModels[0]);
			DistanceFromTop = 0.0; // scroll to top
			DistanceFromBottom = 1.0;
		}
	}

	if (FMath::IsNearlyZero(DistanceFromBottom, 1.0E-8))
	{
		TraceListView->ScrollToBottom();
	}
	else if (FMath::IsNearlyZero(DistanceFromTop, 1.0E-8))
	{
		TraceListView->ScrollToTop();
	}

	// Restore selection.
	if (NewSelectedTraces.Num() > 0)
	{
		TraceListView->ClearSelection();
		TraceListView->SetItemSelection(NewSelectedTraces, true);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FTraceViewModel> STraceStoreWindow::GetSingleSelectedTrace() const
{
	return (TraceListView->GetNumItemsSelected() == 1) ? TraceListView->GetSelectedItems()[0] : nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::TraceList_OnSelectionChanged(TSharedPtr<FTraceViewModel> TraceSession, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		bIsUserSelectedTrace = true;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::TraceList_OnMouseButtonDoubleClick(TSharedPtr<FTraceViewModel> TraceSession)
{
	OpenTraceSession(TraceSession);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState STraceStoreWindow::AutoStart_IsChecked() const
{
	return GetSettings().IsAutoStartAnalysisEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::AutoStart_OnCheckStateChanged(ECheckBoxState NewState)
{
	if (AutoStart_IsChecked() == NewState)
	{
		return;
	}

	GetSettings().SetAndSaveAutoStartAnalysis(!GetSettings().IsAutoStartAnalysisEnabled());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

ECheckBoxState STraceStoreWindow::AutoConnect_IsChecked() const
{
	return GetSettings().IsAutoConnectEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::AutoConnect_OnCheckStateChanged(ECheckBoxState NewState)
{
	if (AutoConnect_IsChecked() == NewState)
	{
		return;
	}

	GetSettings().SetAndSaveAutoConnect(!GetSettings().IsAutoConnectEnabled());

	if (GetSettings().IsAutoConnectEnabled())
	{
		EnableAutoConnect();
	}
	else
	{
		DisableAutoConnect();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::EnableAutoConnect()
{
#if PLATFORM_WINDOWS
	ensure(AutoConnectEvent == nullptr);
	// The event is used by runtime to choose when to try to auto-connect.
	// See FTraceAuxiliary::TryAutoConnect() in \Runtime\Core\Private\ProfilingDebugging\TraceAuxiliary.cpp
	AutoConnectEvent = CreateEvent(NULL, true, false, TEXT("Local\\UnrealInsightsAutoConnect"));
	if (AutoConnectEvent == nullptr || GetLastError() != ERROR_SUCCESS)
	{
		UE_LOGF(LogInsightsFrontend, Warning, "[TraceStore] Failed to create AutoConnect event.");
	}
#elif PLATFORM_MAC || PLATFORM_LINUX
	ensure(AutoConnectEvent == SEM_FAILED);
	sem_unlink("/UnrealInsightsAutoConnect");
	AutoConnectEvent = sem_open("/UnrealInsightsAutoConnect", O_CREAT | O_WRONLY | O_EXCL, 0644, 1);
	if (AutoConnectEvent == SEM_FAILED)
	{
		UE_LOGF(LogInsightsFrontend, Warning, "[TraceStore] Failed to create AutoConnect semaphore: %d", errno);
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::DisableAutoConnect()
{
#if PLATFORM_WINDOWS
	if (AutoConnectEvent != nullptr)
	{
		CloseHandle(AutoConnectEvent);
		AutoConnectEvent = nullptr;
	}
#elif PLATFORM_MAC || PLATFORM_LINUX
	if (AutoConnectEvent != SEM_FAILED)
	{
		sem_close(AutoConnectEvent);
		AutoConnectEvent = SEM_FAILED;
		if (sem_unlink("/UnrealInsightsAutoConnect"))
		{
			UE_LOGF(LogInsightsFrontend, Warning, "[TraceStore] Failed to remove AutoConnect semaphore: %d", errno);
		}
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::CoreTick(float DeltaTime)
{
	// We need to update the trace list, but not too often.
	static uint64 NextTimestamp = 0;
	uint64 Time = FPlatformTime::Cycles64();
	if (Time > NextTimestamp)
	{
		const uint64 WaitTime = static_cast<uint64>(0.5 / FPlatformTime::GetSecondsPerCycle64()); // 500ms
		NextTimestamp = Time + WaitTime;
		RefreshTraceList();

		if (bFilterStatsTextIsDirty)
		{
			UpdateFilterStatsText();
		}
	}

	if (bSetKeyboardFocusOnNextTick)
	{
		bSetKeyboardFocusOnNextTick = false;
		FSlateApplication::Get().ClearKeyboardFocus();
		FSlateApplication::Get().SetKeyboardFocus(TraceListView);
	}

	TickSplashScreenOverlay(DeltaTime);
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EActiveTimerReturnType STraceStoreWindow::UpdateActiveDuration(double InCurrentTime, float InDeltaTime)
{
	DurationActive += InDeltaTime;

	// The window will explicitly unregister this active timer when the mouse leaves.
	return EActiveTimerReturnType::Continue;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	if (!ActiveTimerHandle.IsValid())
	{
		ActiveTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &STraceStoreWindow::UpdateActiveDuration));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	auto PinnedActiveTimerHandle = ActiveTimerHandle.Pin();
	if (PinnedActiveTimerHandle.IsValid())
	{
		UnRegisterActiveTimer(PinnedActiveTimerHandle.ToSharedRef());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	//TODO: Make commands for Rename and Delete.
	//return GetCommandList()->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();

	if (InKeyEvent.GetKey() == EKeys::F5) // refresh metadata for all trace sessions
	{
		StoreBrowser->Refresh();
		SettingsChangeSerial = 0;
		TracesChangeSerial = 0;
		TraceViewModels.Reset();
		TraceViewModelMap.Reset();
		OnTraceListChanged();
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::F2)
	{
		RenameSelectedTrace();
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		DeleteSelectedTraces();
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (DragDropOp.IsValid())
	{
		if (DragDropOp->HasFiles())
		{
			const TArray<FString>& Files = DragDropOp->GetFiles();
			if (Files.Num() == 1)
			{
				const FString DraggedFileExtension = FPaths::GetExtension(Files[0], true);
				if (DraggedFileExtension == TEXT(".utrace"))
				{
					return FReply::Handled();
				}
				if (DraggedFileExtension == TEXT(".csv") || DraggedFileExtension == TEXT(".tsv"))
				{
					return FReply::Handled();
				}
			}
		}
	}

	return SCompoundWidget::OnDragOver(MyGeometry, DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FExternalDragOperation> DragDropOp = DragDropEvent.GetOperationAs<FExternalDragOperation>();
	if (DragDropOp.IsValid())
	{
		if (DragDropOp->HasFiles())
		{
			// For now, only allow a single file.
			const TArray<FString>& Files = DragDropOp->GetFiles();
			if (Files.Num() == 1)
			{
				const FString DraggedFileExtension = FPaths::GetExtension(Files[0], true);
				if (DraggedFileExtension == TEXT(".utrace"))
				{
					OpenTraceFile(Files[0]);
					return FReply::Handled();
				}

				if (DraggedFileExtension == TEXT(".csv") || DraggedFileExtension == TEXT(".tsv"))
				{
					TableImporter->ImportFile(Files[0]);
					return FReply::Handled();
				}
			}
		}
	}

	return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::Open_IsEnabled() const
{
	TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
	return SelectedTrace.IsValid() && SelectedTrace->HasValidId();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::Open_OnClicked()
{
	TSharedPtr<FTraceViewModel> SelectedTrace = GetSingleSelectedTrace();
	OpenTraceSession(SelectedTrace);
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::ShowOpenTraceFileDialog(FString& OutTraceFile) const
{
	if (OpenTraceFileDefaultDirectory.IsEmpty())
	{
		OpenTraceFileDefaultDirectory = FPaths::ConvertRelativePathToFull(TraceStoreConnection->GetStoreDir());
	}

	TArray<FString> OutFiles;
	bool bOpened = false;

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform != nullptr)
	{
		FSlateApplication::Get().CloseToolTip();

		bOpened = DesktopPlatform->OpenFileDialog
		(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("LoadTrace_FileDesc", "Open trace file...").ToString(),
			OpenTraceFileDefaultDirectory,
			TEXT(""),
			LOCTEXT("LoadTrace_FileFilter", "Trace files (*.utrace)|*.utrace|All files (*.*)|*.*").ToString(),
			EFileDialogFlags::None,
			OutFiles
		);
	}

	if (bOpened == true && OutFiles.Num() == 1)
	{
		OutTraceFile = OutFiles[0];
		OpenTraceFileDefaultDirectory = FPaths::GetPath(OutTraceFile);
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OpenTraceFile()
{
	FString TraceFile;
	if (ShowOpenTraceFileDialog(TraceFile))
	{
		OpenTraceFile(TraceFile);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OpenTraceFile(const FString& InTraceFile)
{
	UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Start analysis (in separate process) for trace file: \"%ls\"", *InTraceFile);

	FString CmdLine = TEXT("-OpenTraceFile=\"") + InTraceFile + TEXT("\"");

	FString ExtraCmdParams;
	GetExtraCommandLineParams(ExtraCmdParams);
	CmdLine += ExtraCmdParams;

	FMiscUtils::OpenUnrealInsights(*CmdLine);

	SplashScreenOverlayTraceFile = FPaths::GetBaseFilename(InTraceFile);
	ShowSplashScreenOverlay();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OpenTraceSession(TSharedPtr<FTraceViewModel> InTraceSession)
{
	if (InTraceSession.IsValid() && InTraceSession->HasValidId())
	{
		OpenTraceSession(InTraceSession->GetId());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OpenTraceSession(uint32 InTraceId)
{
	uint32 StoreAddress = 0;
	uint32 StorePort = 0;
	if (!TraceStoreConnection->GetStoreAddressAndPort(StoreAddress, StorePort))
	{
		return;
	}

	UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Start analysis (in separate process) for trace id: 0x%08X", InTraceId);

	FString CmdLine = FString::Printf(TEXT("-OpenTraceId=0x%X -Store=%u.%u.%u.%u:%u"),
		InTraceId,
		(StoreAddress >> 24) & 0xFF,
		(StoreAddress >> 16) & 0xFF,
		(StoreAddress >>  8) & 0xFF,
		(StoreAddress      ) & 0xFF,
		StorePort);

	FString ExtraCmdParams;
	GetExtraCommandLineParams(ExtraCmdParams);
	CmdLine += ExtraCmdParams;

	FMiscUtils::OpenUnrealInsights(*CmdLine);

	TSharedPtr<FTraceViewModel>* TraceSessionPtrPtr = TraceViewModelMap.Find(InTraceId);
	if (TraceSessionPtrPtr)
	{
		FTraceViewModel& TraceSession = **TraceSessionPtrPtr;
		SplashScreenOverlayTraceFile = FPaths::GetBaseFilename(TraceSession.GetUri());
	}
	ShowSplashScreenOverlay();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeTraceListMenu()
{
	FSlateApplication::Get().CloseToolTip();

	RefreshTraceList();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Misc", LOCTEXT("TraceListMenu_Section_Misc", "Misc"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenFileButtonLabel", "Open Trace File..."),
			LOCTEXT("OpenFileButtonTooltip", "Starts analysis for a specified trace file."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
			FUIAction(FExecuteAction::CreateSP(this, &STraceStoreWindow::OpenTraceFile)),
			NAME_None,
			EUserInterfaceActionType::Button);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ImportTableButtonLabel", "Import Table..."),
			LOCTEXT("ImportTableButtonTooltip", "Opens .csv or .tsv file."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.ImportTable"),
			FUIAction(FExecuteAction::CreateLambda([this]{ TableImporter->StartImportProcess(); })),
			NAME_None,
			EUserInterfaceActionType::Button);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DiffTablesButtonLabel", "Diff Tables..."),
			LOCTEXT("DiffTablesButtonTooltip", "Opens two table files in diff mode."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
			FUIAction(FExecuteAction::CreateLambda([this]{ TableImporter->StartDiffProcess(); })),
			NAME_None,
			EUserInterfaceActionType::Button);
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("AvailableTraces", LOCTEXT("TraceListMenu_Section_AvailableTraces", "Top Most Recently Created Traces"));
	{
		// Make a copy of the trace list (to allow list view to be sorted by other criteria).
		TArray<TSharedPtr<FTraceViewModel>> SortedTraces(TraceViewModels);
		Algo::SortBy(SortedTraces, [](const TSharedPtr<FTraceViewModel>& Trace) { return Trace->GetTimestamp(); });

		int32 TraceCountLimit = 10; // top 10

		// Iterate in reverse order as we want most recently created traces first.
		for (int32 TraceIndex = SortedTraces.Num() - 1; TraceIndex >= 0 && TraceCountLimit > 0; --TraceIndex, --TraceCountLimit)
		{
			const FTraceViewModel& Trace = *SortedTraces[TraceIndex];

			FText Label = Trace.GetNameText();
			if (Trace.IsLive())
			{
				Label = FText::Format(LOCTEXT("LiveTraceTextFmt", "{0} (LIVE!)"), Label);
			}

			MenuBuilder.AddMenuEntry(
				Label,
				TAttribute<FText>(), // no tooltip
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &STraceStoreWindow::OpenTraceSession, Trace.GetId())),
				NAME_None,
				EUserInterfaceActionType::Button);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("UnrealTraceServer", LOCTEXT("TraceListMenu_Section_Server", "Server"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("ServerControlLabel", "Unreal Trace Server"),
			LOCTEXT("ServerControlTooltip", "Info and controls for the Unreal Trace Server instances"),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
				{
					for (auto& ServerControl : ServerControls)
					{
						ServerControl.MakeMenu(MenuBuilder);
					}
				}),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Server"));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("DebugOptions", LOCTEXT("TraceListMenu_Section_DebugOptions", "Debug Options"));

	// Enable Automation Tests Option.
	{
		FUIAction ToogleAutomationTestsAction;
		ToogleAutomationTestsAction.ExecuteAction = FExecuteAction::CreateLambda([this]()
			{
				this->SetEnableAutomaticTesting(!this->GetEnableAutomaticTesting());
			});
		ToogleAutomationTestsAction.GetActionCheckState = FGetActionCheckState::CreateLambda([this]()
			{
				return this->GetEnableAutomaticTesting() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			});

		MenuBuilder.AddMenuEntry(
			LOCTEXT("EnableAutomatedTesting", "Enable Session Automation Testing"),
			LOCTEXT("EnableAutomatedTestingDesc", "Activates the automatic test system for new sessions opened from this window."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.TestAutomation"),
			ToogleAutomationTestsAction,
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

	// Enable Debug Tools Option.
	{
		FUIAction ToogleDebugToolsAction;
		ToogleDebugToolsAction.ExecuteAction = FExecuteAction::CreateLambda([this]()
			{
				this->SetEnableDebugTools(!this->GetEnableDebugTools());
			});
		ToogleDebugToolsAction.GetActionCheckState = FGetActionCheckState::CreateLambda([this]()
			{
				return this->GetEnableDebugTools() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			});

		MenuBuilder.AddMenuEntry(
			LOCTEXT("EnableDebugTools", "Enable Debug Tools"),
			LOCTEXT("EnableDebugToolsDesc", "Enables debug tools for new sessions opened from this window."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.Debug"),
			ToogleDebugToolsAction,
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

#if !UE_BUILD_SHIPPING
	// Open Starship Test Suite
	{
		FUIAction OpenStarshipSuiteAction;
		OpenStarshipSuiteAction.ExecuteAction = FExecuteAction::CreateLambda([this]()
			{
				::RestoreStarshipSuite();
			});

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenStarshipSuite", "Starship Test Suite"),
			LOCTEXT("OpenStarshipSuiteDesc", "Opens the Starship UX test suite."),
			FSlateIcon(FInsightsCoreStyle::GetStyleSetName(), "Icons.Test"),
			OpenStarshipSuiteAction,
			NAME_None,
			EUserInterfaceActionType::Button);
	}
#endif // !UE_BUILD_SHIPPING

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakePlatformColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_PlatformFilter", "Platform Filter"));
	BuildPlatformFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakePlatformFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_PlatformFilter", "Platform Filter"));
	BuildPlatformFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildPlatformFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByPlatform->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeAppNameColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_AppNameFilter", "App Name Filter"));
	BuildAppNameFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeAppNameFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_AppNameFilter", "App Name Filter"));
	BuildAppNameFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildAppNameFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByAppName->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBuildConfigColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_BuildConfigFilter", "Build Config Filter"));
	BuildBuildConfigFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBuildConfigFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_BuildConfigFilter", "Build Config Filter"));
	BuildBuildConfigFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildBuildConfigFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByBuildConfig->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBuildTargetColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_BuildTargetFilter", "Build Target Filter"));
	BuildBuildTargetFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBuildTargetFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_BuildTargetFilter", "Build Target Filter"));
	BuildBuildTargetFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildBuildTargetFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByBuildTarget->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBuildVersionColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_BuildVersionFilter", "Build Version Filter"));
	BuildBuildVersionFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBuildVersionFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_BuildVersionFilter", "Build Version Filter"));
	BuildBuildVersionFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildBuildVersionFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByBuildVersion->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBranchColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_BranchFilter", "Branch Filter"));
	BuildBranchFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeBranchFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_BranchFilter", "Branch Filter"));
	BuildBranchFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildBranchFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByBranch->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeChangelistColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_ChangelistFilter", "Changelist Filter"));
	BuildChangelistFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeChangelistFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_ChangelistFilter", "Changelist Filter"));
	BuildChangelistFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildChangelistFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByChangelist->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeEngineVersionColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_EngineVersionFilter", "Engine Version Filter"));
	BuildEngineVersionFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeEngineVersionFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_EngineVersionFilter", "Engine Version Filter"));
	BuildEngineVersionFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildEngineVersionFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByEngineVersion->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeSizeColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_SizeFilter", "Size Filter"));
	BuildSizeFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeSizeFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_SizeFilter", "Size Filter"));
	BuildSizeFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildSizeFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterBySize->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeStatusColumnHeaderMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("ColumnHeaderMenuSection_StatusFilter", "Status Filter"));
	BuildStatusFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> STraceStoreWindow::MakeStatusFilterMenu()
{
	FSlateApplication::Get().CloseToolTip();

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr);

	MenuBuilder.BeginSection("Filter", LOCTEXT("MenuSection_StatusFilter", "Status Filter"));
	BuildStatusFilterSubMenu(MenuBuilder);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::BuildStatusFilterSubMenu(FMenuBuilder& InMenuBuilder)
{
	FilterByStatus->BuildMenu(InMenuBuilder, *this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FText STraceStoreWindow::GetTraceStoreDirectory() const
{
	return FText::FromString(FPaths::ConvertRelativePathToFull(GetStoreDirectory()));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::ExploreTraceStoreDirectory_OnClicked()
{
	FString FullPath(FPaths::ConvertRelativePathToFull(GetStoreDirectory()));
	FPlatformProcess::ExploreFolder(*FullPath);
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool STraceStoreWindow::CanChangeStoreSettings() const
{
	return TraceStoreConnection->CanChangeStoreSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<ITableRow> STraceStoreWindow::TraceDirs_OnGenerateRow(TSharedPtr<FTraceDirectoryModel> Item, const TSharedRef<STableViewBase>& Owner)
{
	return SNew(STableRow<TSharedPtr<FTraceDirectoryModel>>, Owner)
		.Content()
		[
			SNew(STraceDirectoryItem, Item, this)
		];
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::StoreSettingsArea_Toggle() const
{
	if (StoreSettingsArea)
	{
		if (StoreSettingsArea->GetVisibility() == EVisibility::Visible)
		{
			StoreSettingsArea->SetVisibility(EVisibility::Collapsed);
		}
		else
		{
			StoreSettingsArea->SetVisibility(EVisibility::Visible);
		}
	}
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* STraceStoreWindow::StoreSettingsToggle_Icon() const
{
	return (StoreSettingsArea && StoreSettingsArea->GetVisibility() == EVisibility::Visible) ?
		FInsightsFrontendStyle::Get().GetBrush("Icons.Expanded") :
		FInsightsFrontendStyle::Get().GetBrush("Icons.Expand");
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FReply STraceStoreWindow::AddWatchDir_Clicked()
{
	FSlateApplication::Get().CloseToolTip();
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const FString Title = LOCTEXT("AddWatchDirectory_DialogTitle", "Add Monitored Directory").ToString();

		const FString& CurrentStoreDirectory = StoreDirectoryModel.IsEmpty() ? FString() : StoreDirectoryModel.Last()->Path;
		FString SelectedDirectory;
		const bool bHasSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			Title,
			CurrentStoreDirectory,
			SelectedDirectory);

		if (bHasSelected && !FPathViews::Equals(SelectedDirectory, CurrentStoreDirectory))
		{
			FPaths::MakePlatformFilename(SelectedDirectory);

			UE_LOGF(LogInsightsFrontend, Log, "[TraceStore] Adding monitored directory: \"%ls\"...", *SelectedDirectory);

			UE::Trace::FStoreClient* StoreClient = TraceStoreConnection->GetStoreClient();
			if (!StoreClient ||
				!StoreClient->SetStoreDirectories(nullptr, { (*SelectedDirectory) }, {}))
			{
				FMessageLog(LogListingName).Error(LOCTEXT("StoreCommunicationFail", "Failed to change settings on the store service."));
			}
		}
	}
	return FReply::Handled();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FString STraceStoreWindow::GetStoreDirectory() const
{
	return StoreDirectoryModel.IsEmpty() ? FString() : StoreDirectoryModel.Last()->Path;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OpenSettings()
{
#if 0
	MainContentPanel->SetEnabled(false);
	(*OverlaySettingsSlot)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("PopupText.Background"))
		.Padding(8.0f)
		[
			SNew(SInsightsSettings)
			.OnClose(this, &STraceStoreWindow::CloseSettings)
			.SettingPtr(&STraceStoreWindow::GetSettings())
		]
	];
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::CloseSettings()
{
	// Close the profiler settings by simply replacing widget with a null one.
	(*OverlaySettingsSlot)
	[
		SNullWidget::NullWidget
	];
	MainContentPanel->SetEnabled(true);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::GetExtraCommandLineParams(FString& OutParams) const
{
	if (bEnableAutomaticTesting)
	{
		OutParams.Append(TEXT(" -InsightsTest"));
	}
	if (bEnableDebugTools)
	{
		OutParams.Append(TEXT(" -DebugTools"));
	}
	if (bStartProcessWithStompMalloc)
	{
		OutParams.Append(TEXT(" -stompmalloc"));
	}
	if (bDisableFramerateThrottle)
	{
		OutParams.Append(TEXT(" -DisableFramerateThrottle"));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::FilterByNameSearchBox_OnTextChanged(const FText& InFilterText)
{
	FilterByName->SetRawFilterText(InFilterText);
	FilterByNameSearchBox->SetError(FilterByName->GetFilterErrorText());
	OnFilterChanged();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OnFilterChanged()
{
	UpdateFiltering();
	UpdateSorting();
	UpdateTraceListView();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<TSharedPtr<FTraceViewModel>>& STraceStoreWindow::GetAllAvailableTraces() const
{
	return TraceViewModels;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::CreateFilters()
{
	Filters = MakeShared<FTraceViewModelFilterCollection>();

	FilterByName = MakeShared<FTraceTextFilter>(FTraceTextFilter::FItemToStringArray::CreateSP(this, &STraceStoreWindow::HandleItemToStringArray));
	Filters->Add(FilterByName);

	FilterByPlatform = MakeShared<FTraceFilterByPlatform>();
	Filters->Add(FilterByPlatform);

	FilterByAppName = MakeShared<FTraceFilterByAppName>();
	Filters->Add(FilterByAppName);

	FilterByBuildConfig = MakeShared<FTraceFilterByBuildConfig>();
	Filters->Add(FilterByBuildConfig);

	FilterByBuildTarget = MakeShared<FTraceFilterByBuildTarget>();
	Filters->Add(FilterByBuildTarget);

	FilterByBuildVersion = MakeShared<FTraceFilterByBuildVersion>();
	Filters->Add(FilterByBuildVersion);

	FilterByBranch = MakeShared<FTraceFilterByBranch>();
	Filters->Add(FilterByBranch);

	FilterByChangelist = MakeShared<FTraceFilterByChangelist>();
	Filters->Add(FilterByChangelist);

	FilterByEngineVersion = MakeShared<FTraceFilterByEngineVersion>();
	Filters->Add(FilterByEngineVersion);

	FilterBySize = MakeShared<FTraceFilterBySize>();
	Filters->Add(FilterBySize);

	FilterByStatus = MakeShared<FTraceFilterByStatus>();
	Filters->Add(FilterByStatus);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::HandleItemToStringArray(const FTraceViewModel& InTrace, TArray<FString>& OutSearchStrings) const
{
	if (bSearchByCommandLine)
	{
		OutSearchStrings.Add(InTrace.GetCommandLine());
	}
	else
	{
		OutSearchStrings.Add(InTrace.GetName());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::UpdateFiltering()
{
	FilteredTraceViewModels.Reset();

	if (FilterByName->GetRawFilterText().IsEmpty() &&
		FilterByPlatform->IsEmpty() &&
		FilterByAppName->IsEmpty() &&
		FilterByBuildConfig->IsEmpty() &&
		FilterByBuildTarget->IsEmpty() &&
		FilterByBuildVersion->IsEmpty() &&
		FilterByBranch->IsEmpty() &&
		FilterByChangelist->IsEmpty() &&
		FilterByEngineVersion->IsEmpty() &&
		FilterBySize->IsEmpty() &&
		FilterByStatus->IsEmpty())
	{
		// No filtering.
		FilteredTraceViewModels = TraceViewModels;
	}
	else
	{
		for (const TSharedPtr<FTraceViewModel>& Trace : TraceViewModels)
		{
			const bool bIsTraceVisible = Filters->PassesAllFilters(*Trace.Get());
			if (bIsTraceVisible)
			{
				FilteredTraceViewModels.Add(Trace);
			}
		}
	}

	UpdateFilterStatsText();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::UpdateFilterStatsText()
{
	bFilterStatsTextIsDirty = false;

	uint64 FilterdTotalSize = 0;
	for (const TSharedPtr<FTraceViewModel>& Trace : FilteredTraceViewModels)
	{
		FilterdTotalSize += Trace->GetSize();
		if (Trace->IsLive())
		{
			bFilterStatsTextIsDirty = true;
		}
	}

	// When having live sessions, but too many traces, do not further update the stats text on Tick().
	if (FilteredTraceViewModels.Num() > 1000)
	{
		bFilterStatsTextIsDirty = false;
	}

	FNumberFormattingOptions FormattingOptionsSize;
	FormattingOptionsSize.MaximumFractionalDigits = 1;

	if (FilteredTraceViewModels.Num() == TraceViewModels.Num())
	{
		FilterStatsText = FText::Format(LOCTEXT("FilterStatsText_Fmt1", "{0} trace sessions ({1})"),
			FText::AsNumber(TraceViewModels.Num()),
			FText::AsMemory(FilterdTotalSize, &FormattingOptionsSize));
	}
	else
	{
		FilterStatsText = FText::Format(LOCTEXT("FilterStatsText_Fmt2", "{0} / {1} trace sessions ({2})"),
			FText::AsNumber(FilteredTraceViewModels.Num()),
			FText::AsNumber(TraceViewModels.Num()),
			FText::AsMemory(FilterdTotalSize, &FormattingOptionsSize));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

EColumnSortMode::Type STraceStoreWindow::GetSortModeForColumn(const FName ColumnId) const
{
	return ColumnId == SortColumn ? SortMode : EColumnSortMode::None;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortColumn = ColumnId;
	SortMode = InSortMode;
	UpdateSorting();
	UpdateTraceListView();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::UpdateSorting()
{
	if (SortColumn == FTraceListColumns::Date)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetTimestamp() < B->GetTimestamp();
				});
		}
		else
		{
			FilteredTraceViewModels.Sort([](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{ return A->GetTimestamp() > B->GetTimestamp(); });
		}
	}
	else if (SortColumn == FTraceListColumns::Name)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetNameText().CompareTo(B->GetNameText()) < 0;
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return B->GetNameText().CompareTo(A->GetNameText()) < 0;
				});
		}
	}
	else if (SortColumn == FTraceListColumns::Uri)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetUriText().CompareTo(B->GetUriText()) < 0;
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return B->GetUriText().CompareTo(A->GetUriText()) < 0;
				});
		}
	}
	else if (SortColumn == FTraceListColumns::Platform)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					int32 CompareResult = A->GetPlatformText().CompareTo(B->GetPlatformText());
					return CompareResult == 0 ? A->GetTimestamp() < B->GetTimestamp() : CompareResult < 0;
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					int32 CompareResult = B->GetPlatformText().CompareTo(A->GetPlatformText());
					return CompareResult == 0 ? A->GetTimestamp() < B->GetTimestamp() : CompareResult < 0;
				});
		}
	}
	else if (SortColumn == FTraceListColumns::AppName)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					int32 CompareResult = A->GetAppNameText().CompareTo(B->GetAppNameText());
					return CompareResult == 0 ? A->GetTimestamp() < B->GetTimestamp() : CompareResult < 0;
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					int32 CompareResult = B->GetAppNameText().CompareTo(A->GetAppNameText());
					return CompareResult == 0 ? A->GetTimestamp() < B->GetTimestamp() : CompareResult < 0;
				});
		}
	}
	else if (SortColumn == FTraceListColumns::BuildConfig)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetBuildConfiguration() == B->GetBuildConfiguration() ?
						A->GetTimestamp() < B->GetTimestamp() :
						A->GetBuildConfiguration() < B->GetBuildConfiguration();
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetBuildConfiguration() == B->GetBuildConfiguration() ?
						A->GetTimestamp() < B->GetTimestamp() :
						A->GetBuildConfiguration() > B->GetBuildConfiguration();
				});
		}
	}
	else if (SortColumn == FTraceListColumns::BuildTarget)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetBuildTarget() == B->GetBuildTarget() ?
						A->GetTimestamp() < B->GetTimestamp() :
						A->GetBuildTarget() < B->GetBuildTarget();
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetBuildTarget() == B->GetBuildTarget() ?
						A->GetTimestamp() < B->GetTimestamp() :
						A->GetBuildTarget() > B->GetBuildTarget();
				});
		}
	}
	else if (SortColumn == FTraceListColumns::BuildVersion)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetBuildVersionText().CompareTo(B->GetBuildVersionText()) < 0;
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return B->GetBuildVersionText().CompareTo(A->GetBuildVersionText()) < 0;
				});
		}
	}
	else if (SortColumn == FTraceListColumns::BuildBranch)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetBuildBranchText().CompareTo(B->GetBuildBranchText()) < 0;
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return B->GetBuildBranchText().CompareTo(A->GetBuildBranchText()) < 0;
				});
		}
	}
	else if (SortColumn == FTraceListColumns::Changelist)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetChangelist() < B->GetChangelist();
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return B->GetChangelist() < A->GetChangelist();
				});
		}
	}
	else if (SortColumn == FTraceListColumns::EngineVersion)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetEngineVersionNumber() < B->GetEngineVersionNumber();
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return B->GetEngineVersionNumber() < A->GetEngineVersionNumber();
				});
		}
	}
	else if (SortColumn == FTraceListColumns::Size)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetSize() < B->GetSize();
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->GetSize() > B->GetSize();
				});
		}
	}
	else if (SortColumn == FTraceListColumns::Status)
	{
		if (SortMode == EColumnSortMode::Ascending)
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->IsLive() == B->IsLive() ?
						A->GetTimestamp() < B->GetTimestamp() :
						B->IsLive();
				});
		}
		else
		{
			FilteredTraceViewModels.Sort(
				[](const TSharedPtr<FTraceViewModel>& A, const TSharedPtr<FTraceViewModel>& B)
				{
					return A->IsLive() == B->IsLive() ?
						A->GetTimestamp() < B->GetTimestamp() :
						A->IsLive();
				});
		}
	}
	else
	{
		Algo::SortBy(FilteredTraceViewModels,
			[](const TSharedPtr<FTraceViewModel>& Trace)
			{
				return Trace->GetTimestamp();
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::ShowSuccessMessage(FText& InMessage)
{
	FNotificationInfo NotificationInfo(InMessage);
	NotificationInfo.bFireAndForget = false;
	NotificationInfo.bUseLargeFont = false;
	NotificationInfo.bUseSuccessFailIcons = true;
	NotificationInfo.ExpireDuration = 3.0f;
	TSharedRef<SNotificationItem> NotificationItem = NotificationList->AddNotification(NotificationInfo);
	NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
	NotificationItem->ExpireAndFadeout();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::ShowFailMessage(FText& InMessage)
{
	FNotificationInfo NotificationInfo(InMessage);
	NotificationInfo.bFireAndForget = false;
	NotificationInfo.bUseLargeFont = false;
	NotificationInfo.bUseSuccessFailIcons = true;
	NotificationInfo.ExpireDuration = 3.0f;
	TSharedRef<SNotificationItem> NotificationItem = NotificationList->AddNotification(NotificationInfo);
	NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
	NotificationItem->ExpireAndFadeout();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FInsightsFrontendSettings& STraceStoreWindow::GetSettings()
{
	FTraceInsightsFrontendModule& Module = FModuleManager::Get().LoadModuleChecked<FTraceInsightsFrontendModule>("TraceInsightsFrontend");
	return Module.GetSettings();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FInsightsFrontendSettings& STraceStoreWindow::GetSettings() const
{
	FTraceInsightsFrontendModule& Module = FModuleManager::Get().LoadModuleChecked<FTraceInsightsFrontendModule>("TraceInsightsFrontend");
	return Module.GetSettings();
}
////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::AutoStartPlatformFilterBox_OnValueCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	GetSettings().SetAndSaveAutoStartAnalysisPlatform(InText.ToString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void STraceStoreWindow::AutoStartAppNameFilterBox_OnValueCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	GetSettings().SetAndSaveAutoStartAnalysisAppName(InText.ToString());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights

#undef LOCTEXT_NAMESPACE
