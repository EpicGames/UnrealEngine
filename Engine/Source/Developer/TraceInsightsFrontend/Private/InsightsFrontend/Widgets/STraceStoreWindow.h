// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/Ticker.h"
#include "Misc/FilterCollection.h"
#include "Misc/TextFilter.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SListView.h"

// TraceInsightsFrontend
#include "InsightsFrontend/StoreService/TraceServerControl.h"

#if PLATFORM_MAC || PLATFORM_LINUX
#include <semaphore.h> // for sem_t
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////

class FActiveTimerHandle;
class ITableRow;
class SEditableTextBox;
class SNotificationList;
class SScrollBox;
class SSearchBox;
class STableViewBase;
class SVerticalBox;

namespace UE::Trace
{
	class FStoreConnection;
}

namespace UE::Insights
{

class FInsightsFrontendSettings;
class FStoreBrowser;
struct FStoreBrowserTraceInfo;

class FTableImporter;

class FTraceViewModel;
struct FTraceDirectoryModel;

class FTraceFilterByPlatform;
class FTraceFilterByAppName;
class FTraceFilterByBuildConfig;
class FTraceFilterByBuildTarget;
class FTraceFilterByBuildVersion;
class FTraceFilterByBranch;
class FTraceFilterByChangelist;
class FTraceFilterByEngineVersion;
class FTraceFilterBySize;
class FTraceFilterByStatus;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** The filter collection - used for updating the list of trace sessions. */
typedef TFilterCollection<const FTraceViewModel&> FTraceViewModelFilterCollection;

/** The text based filter - used for updating the list of trace sessions. */
typedef TTextFilter<const FTraceViewModel&> FTraceTextFilter;

////////////////////////////////////////////////////////////////////////////////////////////////////

/** Implements the Trace Store window. */
class STraceStoreWindow : public SCompoundWidget
{
	friend class STraceListRow;
	friend class STraceDirectoryItem;

public:
	STraceStoreWindow();
	virtual ~STraceStoreWindow();

	SLATE_BEGIN_ARGS(STraceStoreWindow) {}
	SLATE_END_ARGS()

	/** Constructs this widget. */
	void Construct(const FArguments& InArgs, TSharedRef<UE::Trace::FStoreConnection> InTraceStoreConnection);

	//////////////////////////////////////////////////

	FInsightsFrontendSettings& GetSettings();
	const FInsightsFrontendSettings& GetSettings() const;

	void OpenSettings();
	void CloseSettings();

	//////////////////////////////////////////////////

	void GetExtraCommandLineParams(FString& OutParams) const;

	void SetEnableAutomaticTesting(bool InValue) { bEnableAutomaticTesting = InValue; };
	bool GetEnableAutomaticTesting() const { return bEnableAutomaticTesting; };

	void SetEnableDebugTools(bool InValue) { bEnableDebugTools = InValue; };
	bool GetEnableDebugTools() const { return bEnableDebugTools; };

	void SetStartProcessWithStompMalloc(bool InValue) { bStartProcessWithStompMalloc = InValue; };
	bool GetStartProcessWithStompMalloc() const { return bStartProcessWithStompMalloc; };

	void SetDisableFramerateThrottle(bool InValue) { bDisableFramerateThrottle = InValue; };
	bool GetDisableFramerateThrottle() const { return bDisableFramerateThrottle; };

	void SetDeleteTraceConfirmationWindowVisibility(bool bIsVisible) { bIsDeleteTraceConfirmWindowVisible = bIsVisible; }

	//////////////////////////////////////////////////

	void OnFilterChanged();
	const TArray<TSharedPtr<FTraceViewModel>>& GetAllAvailableTraces() const;

	bool HasValidTraceStoreConnection() const { return TraceStoreConnection.IsValid(); }
	UE::Trace::FStoreConnection& GetTraceStoreConnection() { return *TraceStoreConnection; }
	bool IsConnected() const;

private:
	TSharedRef<SWidget> ConstructFiltersToolbar();
	TSharedRef<SWidget> ConstructSessionsPanel();
	TSharedRef<SWidget> ConstructLoadPanel();
	FText GetConnectionStatusTooltip() const;
	const FSlateBrush* GetConnectionStatusIcon() const;
	EVisibility VisibleIfConnected() const;
	EVisibility VisibleIfNotConnected() const;
	EVisibility HiddenIfNotConnected() const;
	TSharedRef<SWidget> ConstructTraceStoreDirectoryPanel();
	TSharedRef<SWidget> ConstructAutoStartPanel();

	/** Generate a new row for the Traces list view. */
	TSharedRef<ITableRow> TraceList_OnGenerateRow(TSharedPtr<FTraceViewModel> InTrace, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<SWidget> TraceList_GetMenuContent();

	bool CanRenameSelectedTrace() const;
	void RenameSelectedTrace();

	bool CanDeleteSelectedTraces() const;
	void DeleteSelectedTraces();
	bool DeleteTrace(const TSharedPtr<FTraceViewModel>& TraceToDelete);

	bool CanCopyTraceId() const;
	void CopyTraceId();

	bool CanCopyFullPath() const;
	void CopyFullPath();

	bool CanOpenContainingFolder() const;
	void OpenContainingFolder();

	bool HasAnyLiveTrace() const;

	//////////////////////////////////////////////////
	// "Starting analysis..." Splash Screen

	void ShowSplashScreenOverlay();
	void TickSplashScreenOverlay(const float InDeltaTime);
	float SplashScreenOverlayOpacity() const;

	EVisibility SplashScreenOverlay_Visibility() const;
	FSlateColor SplashScreenOverlay_ColorAndOpacity() const;
	FSlateColor SplashScreenOverlay_TextColorAndOpacity() const;
	FText GetSplashScreenOverlayText() const;

	//////////////////////////////////////////////////

	bool Open_IsEnabled() const;
	FReply Open_OnClicked();

	/**
	 * Shows the open file dialog for choosing a trace file.
	 * @param OutTraceFile - The chosen trace file, if successful
	 * @return True, if successful.
	 */
	bool ShowOpenTraceFileDialog(FString& OutTraceFile) const;

	void OpenTraceFile();
	void OpenTraceFile(const FString& InTraceFile);

	void OpenTraceSession(TSharedPtr<FTraceViewModel> InTrace);
	void OpenTraceSession(uint32 InTraceId);

	//////////////////////////////////////////////////
	// Traces

	TSharedRef<SWidget> MakeTraceListMenu();

	TSharedRef<SWidget> MakePlatformColumnHeaderMenu();
	TSharedRef<SWidget> MakePlatformFilterMenu();
	void BuildPlatformFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeAppNameColumnHeaderMenu();
	TSharedRef<SWidget> MakeAppNameFilterMenu();
	void BuildAppNameFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeBuildConfigColumnHeaderMenu();
	TSharedRef<SWidget> MakeBuildConfigFilterMenu();
	void BuildBuildConfigFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeBuildTargetColumnHeaderMenu();
	TSharedRef<SWidget> MakeBuildTargetFilterMenu();
	void BuildBuildTargetFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeBuildVersionColumnHeaderMenu();
	TSharedRef<SWidget> MakeBuildVersionFilterMenu();
	void BuildBuildVersionFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeBranchColumnHeaderMenu();
	TSharedRef<SWidget> MakeBranchFilterMenu();
	void BuildBranchFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeChangelistColumnHeaderMenu();
	TSharedRef<SWidget> MakeChangelistFilterMenu();
	void BuildChangelistFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeEngineVersionColumnHeaderMenu();
	TSharedRef<SWidget> MakeEngineVersionFilterMenu();
	void BuildEngineVersionFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeSizeColumnHeaderMenu();
	TSharedRef<SWidget> MakeSizeFilterMenu();
	void BuildSizeFilterSubMenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> MakeStatusColumnHeaderMenu();
	TSharedRef<SWidget> MakeStatusFilterMenu();
	void BuildStatusFilterSubMenu(FMenuBuilder& InMenuBuilder);

	FReply RefreshTraces_OnClicked();
	FSlateColor GetColorByPath(const FString& Uri);

	void RefreshTraceList();
	void UpdateTrace(FTraceViewModel& InOutTrace, const FStoreBrowserTraceInfo& InSourceTrace);
	void OnTraceListChanged();

	TSharedPtr<FTraceViewModel> GetSingleSelectedTrace() const;

	void TraceList_OnSelectionChanged(TSharedPtr<FTraceViewModel> InTrace, ESelectInfo::Type SelectInfo);
	void TraceList_OnMouseButtonDoubleClick(TSharedPtr<FTraceViewModel> InTrace);

	//////////////////////////////////////////////////
	// Auto Start Analysis

	ECheckBoxState AutoStart_IsChecked() const;
	void AutoStart_OnCheckStateChanged(ECheckBoxState NewState);

	//////////////////////////////////////////////////
	// Auto Connect

	ECheckBoxState AutoConnect_IsChecked() const;
	void AutoConnect_OnCheckStateChanged(ECheckBoxState NewState);

	//////////////////////////////////////////////////
	// Trace Store settings

	FText GetTraceStoreDirectory() const;
	FReply ExploreTraceStoreDirectory_OnClicked();

	FString GetStoreDirectory() const;

	bool CanChangeStoreSettings() const;

	TSharedRef<ITableRow> TraceDirs_OnGenerateRow(TSharedPtr<FTraceDirectoryModel> Item, const TSharedRef<STableViewBase>& Owner);

	FReply StoreSettingsArea_Toggle() const;
	const FSlateBrush* StoreSettingsToggle_Icon() const;

	FReply AddWatchDir_Clicked();

	friend class STraceWatchDirTableRow;

	//////////////////////////////////////////////////

	/** Updates the amount of time the profiler has been active. */
	EActiveTimerReturnType UpdateActiveDuration(double InCurrentTime, float InDeltaTime);

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is NOT bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is NOT bubbled.
	 *
	 * @param MouseEvent Information about the input event
	 */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/**
	 * Called after a key is pressed when this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/**
	 * Called during drag and drop when the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)  override;

	/** Updates this class, done through FCoreTicker. Updates also when the page is not visible, unlike the Tick() function */
	bool CoreTick(float DeltaTime);

	//////////////////////////////////////////////////
	// Filtering

	void FilterByNameSearchBox_OnTextChanged(const FText& InFilterText);

	FText GetFilterStatsText() const { return FilterStatsText; }

	void CreateFilters();

	/**
	 * Populates OutSearchStrings with the strings that should be used in searching.
	 *
	 * @param GroupOrStatNodePtr - the group and stat node to get a text description from.
	 * @param OutSearchStrings   - an array of strings to use in searching.
	 *
	 */
	void HandleItemToStringArray(const FTraceViewModel& InTrace, TArray<FString>& OutSearchStrings) const;

	void UpdateFiltering();

	void UpdateFilterStatsText();

	//////////////////////////////////////////////////
	// Sorting

	EColumnSortMode::Type GetSortModeForColumn(const FName ColumnId) const;
	void OnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	void UpdateSorting();

	//////////////////////////////////////////////////

	void UpdateTraceListView();

	void ShowSuccessMessage(FText& InMessage);
	void ShowFailMessage(FText& InMessage);

	void EnableAutoConnect();
	void DisableAutoConnect();

	void AutoStartPlatformFilterBox_OnValueCommitted(const FText& InText, ETextCommit::Type InCommitType);
	void AutoStartAppNameFilterBox_OnValueCommitted(const FText& InText, ETextCommit::Type InCommitType);

private:
	FTickerDelegate OnTick;
	FTSTicker::FDelegateHandle OnTickHandle;

	/** The handle to the active update duration tick */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** The number of seconds the profiler has been active */
	float DurationActive = 0.0f;

	//////////////////////////////////////////////////
	// UI Layout

	TSharedPtr<SVerticalBox> MainContentPanel;

	/** Widget for the non-intrusive notifications. */
	TSharedPtr<SNotificationList> NotificationList;

	/** Overlay slot which contains the profiler settings widget. */
	SOverlay::FOverlaySlot* OverlaySettingsSlot = nullptr;

	//////////////////////////////////////////////////

	TSharedPtr<UE::Trace::FStoreConnection> TraceStoreConnection;
	TUniquePtr<FStoreBrowser> StoreBrowser;
	uint32 SettingsChangeSerial = 0;
	uint32 TracesChangeSerial = 0;

	TArray<TSharedPtr<FTraceDirectoryModel>> StoreDirectoryModel;
	TArray<TSharedPtr<FTraceDirectoryModel>> WatchDirectoriesModel;

	TArray<TSharedPtr<FTraceViewModel>> TraceViewModels; // all available trace view models
	TArray<TSharedPtr<FTraceViewModel>> FilteredTraceViewModels; // the filtered list of trace view models
	TMap<uint32, TSharedPtr<FTraceViewModel>> TraceViewModelMap;

	TSharedPtr<SEditableTextBox> StoreDirTextBox;
	TSharedPtr<SEditableTextBox> StoreHostTextBox;
	TSharedPtr<STableViewBase> StoreDirListView;
	TSharedPtr<SScrollBox> StoreSettingsArea;
	TSharedPtr<STableViewBase> WatchDirsListView;
	TSharedPtr<SListView<TSharedPtr<FTraceViewModel>>> TraceListView;

	bool bIsUserSelectedTrace = false;

	/** Parameter that controls the visibility of the confirmation window in case the trace is deleted. */
	bool bIsDeleteTraceConfirmWindowVisible = true;

	//////////////////////////////////////////////////
	// Filtering

	TSharedPtr<FTraceViewModelFilterCollection> Filters;

	bool bSearchByCommandLine = false;
	TSharedPtr<SSearchBox> FilterByNameSearchBox;
	TSharedPtr<FTraceTextFilter> FilterByName;

	TSharedPtr<FTraceFilterByPlatform> FilterByPlatform;
	TSharedPtr<FTraceFilterByAppName> FilterByAppName;
	TSharedPtr<FTraceFilterByBuildConfig> FilterByBuildConfig;
	TSharedPtr<FTraceFilterByBuildTarget> FilterByBuildTarget;
	TSharedPtr<FTraceFilterByBuildVersion> FilterByBuildVersion;
	TSharedPtr<FTraceFilterByBranch> FilterByBranch;
	TSharedPtr<FTraceFilterByChangelist> FilterByChangelist;
	TSharedPtr<FTraceFilterByEngineVersion> FilterByEngineVersion;
	TSharedPtr<FTraceFilterBySize> FilterBySize;
	TSharedPtr<FTraceFilterByStatus> FilterByStatus;

	bool bFilterStatsTextIsDirty = true;
	FText FilterStatsText;

	//////////////////////////////////////////////////
	// Sorting

	FName SortColumn;
	EColumnSortMode::Type SortMode = EColumnSortMode::None;

	//////////////////////////////////////////////////
	// Auto-start functionality

	TArray<uint32> AutoStartedSessions; // tracks sessions that were auto started (in order to not start them again)

	TSharedPtr<SSearchBox> AutoStartPlatformFilter;
	TSharedPtr<SSearchBox> AutoStartAppNameFilter;
	EBuildConfiguration AutoStartConfigurationTypeFilter = EBuildConfiguration::Unknown;
	EBuildTargetType AutoStartTargetTypeFilter = EBuildTargetType::Unknown;

	//////////////////////////////////////////////////

	FString SplashScreenOverlayTraceFile;
	float SplashScreenOverlayFadeTime = 0.0f;

	mutable FString OpenTraceFileDefaultDirectory;

	bool bEnableAutomaticTesting = false;
	bool bEnableDebugTools = false;
	bool bStartProcessWithStompMalloc = false;

	bool bDisableFramerateThrottle = false;
	bool bSetKeyboardFocusOnNextTick = false;

	TArray<FTraceServerControl> ServerControls;

#if PLATFORM_WINDOWS
	void* AutoConnectEvent = nullptr;
#elif PLATFORM_MAC || PLATFORM_LINUX
	sem_t* AutoConnectEvent = SEM_FAILED;
#endif

	static FName LogListingName;
	TSharedRef<FTableImporter> TableImporter;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace UE::Insights
