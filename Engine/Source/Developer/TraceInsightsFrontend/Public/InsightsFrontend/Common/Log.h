// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogMacros.h"

TRACEINSIGHTSFRONTEND_API DECLARE_LOG_CATEGORY_EXTERN(LogInsightsFrontend, Verbose, All);

LLM_DECLARE_TAG_API(InsightsFrontend, TRACEINSIGHTSFRONTEND_API);
