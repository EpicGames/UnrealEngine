// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

#define INSIGHTS_ENABLE_AUTOMATION !WITH_EDITOR

namespace UE::Insights
{

/** Contains parameters that are passed to the CreateSessionBrowser function to control specific behaviors. */
struct FCreateFrontendWindowParams
{
	bool bAllowDebugTools = false;
	bool bInitializeTesting = false;
	bool bStartProcessWithStompMalloc = false;
	bool bDisableFramerateThrottle = false;
	bool bAutoQuit = false;
};

struct FInsightsFrontendTabs
{
	static const FName TraceStoreTabId;
	static const FName ConnectionTabId;
};

class STraceStoreWindow;
class SConnectionWindow;

} // namespace UE::Insights

/** Interface for TraceInsightsCore module. */
class ITraceInsightsFrontendModule : public IModuleInterface
{
public:
	virtual ~ITraceInsightsFrontendModule() = default;

	virtual bool ConnectToStore(const TCHAR* InStoreHost, uint32 InStorePort = 0) = 0;

	virtual void CreateFrontendWindow(const UE::Insights::FCreateFrontendWindowParams& Params) = 0;

	virtual TSharedPtr<UE::Insights::STraceStoreWindow> GetTraceStoreWindow() const = 0;
	virtual TSharedPtr<UE::Insights::SConnectionWindow> GetConnectionWindow() const = 0;

	/**
	 * Called to run automation tests in Unreal Insights.
	 */
	virtual void RunAutomationTests(const FString& InCmd) = 0;
};
