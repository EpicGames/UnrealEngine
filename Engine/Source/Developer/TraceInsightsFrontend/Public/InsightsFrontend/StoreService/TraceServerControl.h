// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#include <atomic>

#define UE_API TRACEINSIGHTSFRONTEND_API

class FMenuBuilder;

namespace UE::Trace { class FStoreClient; }

namespace UE::Insights
{

class FTraceServerControl
{
public:
	UE_API FTraceServerControl(const TCHAR* Host, uint32 Port = 0, FName StyleSet = NAME_None);
	UE_API ~FTraceServerControl();

	UE_API void MakeMenu(FMenuBuilder& Builder);

private:
	enum class EState : uint8
	{
		NotConnected,
		Connecting,
		Connected,
		CheckStatus,
		Command
	};

	UE_API bool ChangeState(EState Expected, EState ChangeTo, uint32 Attempts = 1);

	UE_API void TriggerStatusUpdate();
	UE_API void UpdateStatus();
	UE_API void ResetStatus();

	bool CanServerBeStarted() const { return !bIsCancelRequested && bIsLocalHost && State.load(std::memory_order_relaxed) == EState::NotConnected; }
	bool CanServerBeStopped() const { return !bIsCancelRequested && bIsLocalHost && State.load(std::memory_order_relaxed) == EState::Connected; }
	bool AreControlsEnabled() const { return !bIsCancelRequested && bIsLocalHost && State.load(std::memory_order_relaxed) == EState::Connected; }
	bool IsSponsored() const { return bSponsored.load(std::memory_order_relaxed); }

	UE_API void OnStart_Clicked();
	UE_API void OnStop_Clicked();
	UE_API void OnSponsored_Changed();

	std::atomic<EState> State = EState::NotConnected;

	std::atomic<bool> bCanServerBeStarted = false;
	std::atomic<bool> bCanServerBeStopped = false;
	std::atomic<bool> bSponsored = false;
	std::atomic<bool> bIsCancelRequested = false;

	FCriticalSection AsyncTaskLock;
	FCriticalSection StringsLock;
	FString StatusString;

	FString Host;
	uint32 Port = 0;
	FName StyleSet;
	bool bIsLocalHost = false;
	TUniquePtr<UE::Trace::FStoreClient> Client;

	friend const TCHAR* LexState(EState);
};

} // namespace UE::Insights

#undef UE_API
