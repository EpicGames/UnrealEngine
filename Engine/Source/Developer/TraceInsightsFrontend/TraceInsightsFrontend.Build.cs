// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceInsightsFrontend : ModuleRules
{
	public TraceInsightsFrontend(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework", // for RestoreStarshipSuite()
				"ApplicationCore",
				"AutomationDriver",
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"InputCore",
				"Slate",
				"SlateCore",
				"Sockets",
				"TraceAnalysis",
				"TraceInsightsCore",
				"TraceServices",
				"WorkspaceMenuStructure",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine",
				}
			);
		}

		// Modules required for running automation
		if (!Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AutomationWorker",
					"AutomationController",
					"AutomationWindow",
					"SessionServices",
				}
			);
		}
	}
}
